{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Level Sequence Editor",
	"Description" : "Content Editor for LevelSequence Assets.",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed": false,
	"SupportedPrograms": ["LiveLinkHub"],
	"Modules" :
	[
		{
			"Name" : "LevelSequenceEditor",
			"Type": "Editor",
			"ProgramAllowList": ["LiveLinkHub"],
			"LoadingPhase" : "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "SequencerScripting",
			"Enabled": true
		}
	]
}