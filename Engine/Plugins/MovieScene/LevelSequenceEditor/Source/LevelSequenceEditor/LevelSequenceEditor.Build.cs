// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class LevelSequenceEditor : ModuleRules
{
	public LevelSequenceEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePaths.AddRange(
			new string[] {
                "../../../../Source/Editor/UnrealEd/Private", // TODO: Fix this, for now it's needed for the fbx exporter
				}
			);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
				"AssetTools",
				"PlacementMode",
			}
        );

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"SequencerScripting",
				"SequencerScriptingEditor",
			}
		);
        
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AppFramework",
                "LevelSequence",
				"BlueprintGraph",
                "CinematicCamera",
				"ClassViewer",
				"Constraints",
				"ContentBrowser",
				"Core",
				"CoreUObject",
				"CurveEditor",
				"EditorFramework",
				"EditorSubsystem",
				"Engine",
                "InputCore",
                "Kismet",
                "KismetCompiler",
				"LevelEditor",
				"MovieScene",
                "MovieSceneTools",
				"MovieSceneTracks",
				"Projects",
                "PropertyEditor",
				"SequencerCore",
				"Sequencer",
                "Slate",
                "SlateCore",
				"SceneOutliner",
                "UnrealEd",
                "VREditor",
				"TimeManagement",
				"ToolMenus",
				"ToolWidgets",
				"AssetDefinition",
				"UniversalObjectLocator"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetTools",
				"PlacementMode",
                "Settings",
                "MovieSceneCaptureDialog",
				"DesktopPlatform",
			}
		);

        PrivateIncludePaths.AddRange(
            new string[] {
				"LevelSequenceEditor/Private/AssetTools",
				"LevelSequenceEditor/Private/Factories",
                "LevelSequenceEditor/Private/Misc",
				"LevelSequenceEditor/Private/Styles",
			}
        );

		AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");
	}
}
