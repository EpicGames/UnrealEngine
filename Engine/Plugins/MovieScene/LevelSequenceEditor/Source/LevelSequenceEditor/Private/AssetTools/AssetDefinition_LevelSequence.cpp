// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTools/AssetDefinition_LevelSequence.h"
#include "LevelSequenceEditorToolkit.h"
#include "LevelSequenceEditorStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_LevelSequence)

#define LOCTEXT_NAMESPACE "AssetTypeActions"

FAssetOpenSupport UAssetDefinition_LevelSequence::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit, EToolkitMode::WorldCentric); 
}

EAssetCommandResult UAssetDefinition_LevelSequence::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	UWorld* WorldContext = nullptr;
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::Editor)
		{
			WorldContext = Context.World();
			break;
		}
	}

	if (!ensure(WorldContext))
	{
		return EAssetCommandResult::Handled;
	}

	for (ULevelSequence* LevelSequence : OpenArgs.LoadObjects<ULevelSequence>())
	{
		TSharedRef<FLevelSequenceEditorToolkit> Toolkit = MakeShareable(new FLevelSequenceEditorToolkit(FLevelSequenceEditorStyle::Get()));
		Toolkit->Initialize(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, LevelSequence);
	}

	return EAssetCommandResult::Handled;
}

#undef LOCTEXT_NAMESPACE
