// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "CinematicLevelViewportToolbarContext.generated.h"

class SCinematicLevelViewport;

UCLASS()
class UCinematicLevelViewportToolbarContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<SCinematicLevelViewport> CinematicLevelViewport;
};
