// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicViewportCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "InputCoreTypes.h"

#define LOCTEXT_NAMESPACE "CinematicViewportCommands"

void FCinematicViewportCommands::RegisterCommands()
{
	UI_COMMAND( Disabled, "Disabled", "Disable the composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadZero) );
	UI_COMMAND( Grid3x3, "Grid 3x3", "Enable the grid (3x3) composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadSix) );
	UI_COMMAND( Grid2x2, "Grid 2x2", "Enable the grid (2x2) composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadSeven) );
	UI_COMMAND( Crosshair, "Crosshair", "Enable the crosshair composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadEight) );
	UI_COMMAND( Rabatment, "Rabatment", "Enable the rabatment composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadNine));
	UI_COMMAND( ActionSafe, "Action Safe", "Enable the action safe composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadOne));
	UI_COMMAND( TitleSafe, "Title Safe", "Enable the title safe composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadTwo));
	UI_COMMAND( CustomSafe, "Custom Safe", "Enable the custom safe composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadThree));
	UI_COMMAND( Letterbox, "Letterbox", "Enable the letterbox composition overlay", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::NumPadFour));
}

#undef LOCTEXT_NAMESPACE
