// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FUICommandInfo;

class FCinematicViewportCommands : public TCommands<FCinematicViewportCommands>
{

public:
	FCinematicViewportCommands() : TCommands<FCinematicViewportCommands>
	(
		"Cinematic Viewport",
		NSLOCTEXT("Contexts", "CinematicViewport", "Cinematic Viewport"),
		NAME_None,
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{}
	
	/** Disabled */
	TSharedPtr<FUICommandInfo> Disabled;

	/** Grid 2x2 */
	TSharedPtr<FUICommandInfo> Grid2x2;

	/** Grid 3x3 */
	TSharedPtr<FUICommandInfo> Grid3x3;

	/** Crosshair */
	TSharedPtr<FUICommandInfo> Crosshair;

	/** Rabatment */
	TSharedPtr<FUICommandInfo> Rabatment;

	/** ActionSafe */
	TSharedPtr<FUICommandInfo> ActionSafe;

	/** TitleSafe */
	TSharedPtr<FUICommandInfo> TitleSafe;

	/** CustomSafe */
	TSharedPtr<FUICommandInfo> CustomSafe;

	/** Letterbox */
	TSharedPtr<FUICommandInfo> Letterbox;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};
