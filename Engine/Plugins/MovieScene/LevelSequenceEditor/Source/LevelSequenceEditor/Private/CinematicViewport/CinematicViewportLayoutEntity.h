// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/SlateApplication.h"
#include "LevelViewportLayout.h"
#include "SLevelViewport.h"
#include "CinematicViewport/SCinematicLevelViewport.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "ILevelEditor.h"

class FCinematicViewportLayoutEntity : public ILevelViewportLayoutEntity
{
public:
	FCinematicViewportLayoutEntity(const FAssetEditorViewportConstructionArgs& Args, TSharedPtr<ILevelEditor> InLevelEditor)
		: Widget(
			SNew(SCinematicLevelViewport)
			.ParentLayout(Args.ParentLayout)
			.ParentLevelEditor(InLevelEditor)
			.LayoutName(Args.ConfigKey)
			)
	{}

	virtual TSharedRef<SWidget> AsWidget() const override { return Widget; }
	virtual TSharedPtr<SLevelViewport> AsLevelViewport() const { return Widget->GetLevelViewport(); }

	FName GetType() const
	{
		static FName CinematicName("Cinematic");
		return CinematicName;
	}

	FLevelEditorViewportClient& GetLevelViewportClient() const { return Widget->GetLevelViewport()->GetLevelViewportClient(); }
	bool IsPlayInEditorViewportActive() const { return Widget->GetLevelViewport()->IsPlayInEditorViewportActive(); }
	void RegisterGameViewportIfPIE(){ return Widget->GetLevelViewport()->RegisterGameViewportIfPIE(); }
	void SetKeyboardFocus(){ FSlateApplication::Get().SetKeyboardFocus(Widget->GetLevelViewport()); }
	void OnLayoutDestroyed()
	{
		if (IsPlayInEditorViewportActive() || GetLevelViewportClient().IsSimulateInEditorViewport() )
		{
			GUnrealEd->EndPlayMap();
		}
	}
	void SaveConfig(const FString& ConfigString){ return Widget->GetLevelViewport()->SaveConfig(ConfigString); }
	void TakeHighResScreenShot() const {}

protected:

	/** This entity's widget */
	TSharedRef<SCinematicLevelViewport> Widget;
};
