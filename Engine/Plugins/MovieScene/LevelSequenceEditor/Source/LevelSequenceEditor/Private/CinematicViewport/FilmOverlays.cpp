// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicViewport/FilmOverlays.h"
#include "CinematicViewportCommands.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SHeader.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styles/LevelSequenceEditorStyle.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Colors/SColorPicker.h"
#include "FilmOverlayToolkit.h"

#define LOCTEXT_NAMESPACE "LevelSequenceEditorFilmOverlays"

DECLARE_DELEGATE_OneParam(FOnColorPicked, const FLinearColor&);

namespace WidgetHelpers
{
	TSharedRef<SWidget> CreateColorWidget(FLinearColor* Color, FOnColorPicked OnColorPicked = FOnColorPicked())
	{
		auto GetValue = [=]{
			return *Color;
		};

		auto SetValue = [=](FLinearColor NewColor){
			*Color = NewColor;
			OnColorPicked.ExecuteIfBound(NewColor);
		};

		auto OnGetMenuContent = [=]() -> TSharedRef<SWidget> {
			// Open a color picker
			return SNew(SColorPicker)
			.TargetColorAttribute_Lambda(GetValue)
			.UseAlpha(true)
			.DisplayInlineVersion(true)
			.OnColorCommitted_Lambda(SetValue);
		};

		return SNew(SComboButton)
			.ContentPadding(0)
			.HasDownArrow(false)
			.ButtonStyle(FAppStyle::Get(), "Sequencer.AnimationOutliner.ColorStrip")
			.OnGetMenuContent_Lambda(OnGetMenuContent)
			.CollapseMenuOnParentFocus(true)
			.ButtonContent()
			[
				SNew(SColorBlock)
				.Color_Lambda(GetValue)
				.ShowBackgroundForAlpha(true)
				.Size(FVector2D(10.0f, 10.0f))
			];
	}

	template<typename T>
	TSharedRef<SWidget> CreateSpinBox(T* Value, T Min, T Max, TSharedPtr<INumericTypeInterface<T>> TypeInterface = nullptr)
	{
		auto GetValue = [=]{ return *Value; };
		auto SetValue = [=](T NewValue){ *Value = NewValue; };
		auto SetValueCommitted = [=](T NewValue, ETextCommit::Type){ *Value = NewValue; };

		return SNew(SSpinBox<T>)
			.MinValue(Min)
			.MaxValue(Max)
			.Value_Lambda(GetValue)
			.OnValueChanged_Lambda(SetValue)
			.OnValueCommitted_Lambda(SetValueCommitted)
			.TypeInterface(TypeInterface);
	}
}

struct FFilmOverlay_None : IFilmOverlay
{
	FText GetDisplayName() const { return LOCTEXT("OverlayDisabled", "Disabled"); }
	
	FText GetToolTip() const
	{
		return FCinematicViewportCommands::Get().Disabled.Get()->GetInputText();
	}

	const FSlateBrush* GetThumbnail() const { return FLevelSequenceEditorStyle::Get()->GetBrush("FilmOverlay.Disabled"); }

	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
	}
};

struct FFilmOverlay_Grid : IFilmOverlay
{
	FFilmOverlay_Grid(int32 InNumDivsH, int32 InNumDivsV)
		: NumDivsH(InNumDivsH)
		, NumDivsV(InNumDivsV)
	{
		BrushName = *FString::Printf(TEXT("FilmOverlay.Grid%dx%d"), NumDivsH, NumDivsV);
	}

	FText GetDisplayName() const
	{
		return FText::Format(
			LOCTEXT("GridNameFormat", "Grid ({0}x{1})"),
			FText::AsNumber(NumDivsH),
			FText::AsNumber(NumDivsV)
		);
	}
	
	FText GetToolTip() const
	{
		if (NumDivsH == 2 && NumDivsV == 2)
		{
			return FCinematicViewportCommands::Get().Grid2x2.Get()->GetInputText();
		}
		else if (NumDivsH == 3 && NumDivsV == 3)
		{
			return FCinematicViewportCommands::Get().Grid3x3.Get()->GetInputText();
		}
		else
		{
			return GetDisplayName();
		}
	}

	const FSlateBrush* GetThumbnail() const
	{
		return FLevelSequenceEditorStyle::Get()->GetBrush(BrushName);
	}

	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		const int32 IntervalH = AllottedGeometry.GetLocalSize().X / NumDivsH;
		const int32 IntervalV = AllottedGeometry.GetLocalSize().Y / NumDivsV;

		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		for (int32 OffsetH = 1; OffsetH < NumDivsH; ++OffsetH)
		{
			LinePoints[0] = FVector2D(IntervalH*OffsetH, 0.f);
			LinePoints[1] = FVector2D(IntervalH*OffsetH, AllottedGeometry.Size.Y);

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				Tint,
				false
				);
		}

		for (int32 OffsetV = 1; OffsetV < NumDivsV; ++OffsetV)
		{
			LinePoints[0] = FVector2D(0.f, IntervalV*OffsetV);
			LinePoints[1] = FVector2D(AllottedGeometry.GetLocalSize().X, IntervalV*OffsetV);

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				Tint,
				false
				);
		}

	}
private:
	FName BrushName;
	int32 NumDivsH;
	int32 NumDivsV;
};

struct FFilmOverlay_Rabatment : IFilmOverlay
{
	FText GetDisplayName() const { return LOCTEXT("RabatmentName", "Rabatment"); }

	FText GetToolTip() const { return FCinematicViewportCommands::Get().Rabatment.Get()->GetInputText(); }

	const FSlateBrush* GetThumbnail() const { return FLevelSequenceEditorStyle::Get()->GetBrush("FilmOverlay.Rabatment"); }

	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		FVector2D Position(AllottedGeometry.Size.X / 2, AllottedGeometry.Size.Y / 2);
		float Size = FMath::Min(AllottedGeometry.Size.X, AllottedGeometry.Size.Y) * .1f;

		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		LinePoints[0] = FVector2D(AllottedGeometry.GetLocalSize().Y, 0.f);
		LinePoints[1] = FVector2D(AllottedGeometry.GetLocalSize().Y, AllottedGeometry.Size.Y);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint,
			false
			);

		LinePoints[0] = FVector2D(AllottedGeometry.GetLocalSize().X - AllottedGeometry.GetLocalSize().Y, 0.f);
		LinePoints[1] = FVector2D(AllottedGeometry.GetLocalSize().X - AllottedGeometry.GetLocalSize().Y, AllottedGeometry.GetLocalSize().Y);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint,
			false
			);

	}
};

struct FFilmOverlay_Crosshair : IFilmOverlay
{
	FText GetDisplayName() const { return LOCTEXT("CrosshairName", "Crosshair"); }
	FText GetToolTip() const { return FCinematicViewportCommands::Get().Crosshair.Get()->GetInputText(); }
	const FSlateBrush* GetThumbnail() const { return FLevelSequenceEditorStyle::Get()->GetBrush("FilmOverlay.Crosshair"); }
	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		FVector2D Position(AllottedGeometry.Size.X / 2, AllottedGeometry.Size.Y / 2);
		float Size = FMath::Min(AllottedGeometry.Size.X, AllottedGeometry.Size.Y) * .1f;

		TArray<FVector2D> LinePoints;
		LinePoints.SetNum(2);

		LinePoints[0] = FVector2D(Position.X, Position.Y - Size);
		LinePoints[1] = FVector2D(Position.X, Position.Y - Size * .25f);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint
			);
		LinePoints[0] = FVector2D(Position.X, Position.Y + Size);
		LinePoints[1] = FVector2D(Position.X, Position.Y + Size * .25f);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint,
			false
			);

		LinePoints[0] = FVector2D(Position.X - Size, Position.Y);
		LinePoints[1] = FVector2D(Position.X - Size * .25f, Position.Y);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint
			);
		LinePoints[0] = FVector2D(Position.X + Size, Position.Y);
		LinePoints[1] = FVector2D(Position.X + Size * .25f, Position.Y);
		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint,
			false
			);
	}
};

struct FFilmOverlay_SafeFrame : IFilmOverlay
{
	FFilmOverlay_SafeFrame(FText InDisplayName, int32 InSizePercentage, const FLinearColor& InColor)
	{
		DisplayName = InDisplayName;
		SizePercentage = InSizePercentage;

		Tint = InColor;
	}

	FText GetDisplayName() const { return DisplayName; }
	FText GetToolTip() const
	{
		if (GetDisplayName().EqualTo(FCinematicViewportCommands::Get().ActionSafe->GetLabel()))
		{
			return FCinematicViewportCommands::Get().ActionSafe.Get()->GetInputText();
		}
		else if (GetDisplayName().EqualTo(FCinematicViewportCommands::Get().TitleSafe->GetLabel()))
		{
			return FCinematicViewportCommands::Get().TitleSafe.Get()->GetInputText();
		}
		else if (GetDisplayName().EqualTo(FCinematicViewportCommands::Get().CustomSafe->GetLabel()))
		{
			return FCinematicViewportCommands::Get().CustomSafe.Get()->GetInputText();
		}
		return GetDisplayName();
	}

	const FSlateBrush* GetThumbnail() const { return nullptr; }

	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		FVector2D TopLeft = AllottedGeometry.GetLocalSize() * ((100.f - SizePercentage) * .5f) / 100.f;
		FVector2D BottomRight = AllottedGeometry.GetLocalSize() - TopLeft;

		TArray<FVector2D> LinePoints;
		LinePoints.Add(FVector2D(TopLeft.X,			TopLeft.Y));
		LinePoints.Add(FVector2D(BottomRight.X,		TopLeft.Y));
		LinePoints.Add(FVector2D(BottomRight.X,		BottomRight.Y));
		LinePoints.Add(FVector2D(TopLeft.X,			BottomRight.Y));
		LinePoints.Add(FVector2D(TopLeft.X-1,		TopLeft.Y-1));

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			Tint,
			false
			);
	}

	virtual TSharedPtr<SWidget> ConstructSettingsWidget() override
	{
		struct FPercentageInterface : TDefaultNumericTypeInterface<int32>
		{
			virtual FString ToString(const int32& Value) const override
			{
				return TDefaultNumericTypeInterface::ToString(Value) + TEXT("%");
			}
			virtual TOptional<int32> FromString(const FString& InString, const int32& InExistingValue) override
			{
				return TDefaultNumericTypeInterface::FromString(InString.Replace(TEXT("%"), TEXT("")), InExistingValue);
			}
			virtual bool IsCharacterValid(TCHAR InChar) const override
			{
				return InChar == '%' || TDefaultNumericTypeInterface::IsCharacterValid(InChar);
			}
		};

		return SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				WidgetHelpers::CreateSpinBox<int32>(&SizePercentage, 1, 99, MakeShareable(new FPercentageInterface))
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0.f, 1.f, 0.f, 0.f))
			.AutoHeight()
			[
				WidgetHelpers::CreateColorWidget(&Tint)
			];
	}

private:
	int32 SizePercentage;
	FText DisplayName;
};


struct FFilmOverlay_LetterBox : IFilmOverlay
{
	FFilmOverlay_LetterBox()
	{
		Ratio1 = 2.35f;
		Ratio2 = 1.f;
		Color = FLinearColor(0.f, 0.f, 0.f, .5f);
	}

	FText GetDisplayName() const { return LOCTEXT("LetterboxMask", "Letterbox Mask"); }
	FText GetToolTip() const { return FCinematicViewportCommands::Get().Letterbox.Get()->GetInputText(); }

	const FSlateBrush* GetThumbnail() const { return nullptr; }

	void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
	{
		const FSlateBrush* Brush = FAppStyle::GetBrush("WhiteBrush");

		const float DesiredRatio = Ratio1 / Ratio2;
		const float CurrentRatio = AllottedGeometry.GetLocalSize().X / AllottedGeometry.GetLocalSize().Y;

		if (CurrentRatio > DesiredRatio)
		{
			// vertical letterbox mask
			FVector2D LetterBoxSize((AllottedGeometry.GetLocalSize().X - AllottedGeometry.GetLocalSize().Y * DesiredRatio) * .5f, AllottedGeometry.GetLocalSize().Y);
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(LetterBoxSize, FSlateLayoutTransform(FVector2D(0.f, 0.f))),
				Brush,
				ESlateDrawEffect::None,
				Color
				);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(LetterBoxSize, FSlateLayoutTransform(FVector2D(AllottedGeometry.GetLocalSize().X - LetterBoxSize.X, 0.f))),
				Brush,
				ESlateDrawEffect::None,
				Color
				);

		}
		else if (CurrentRatio < DesiredRatio)
		{
			// horizontal letterbox mask
			FVector2D LetterBoxSize(AllottedGeometry.GetLocalSize().X, (AllottedGeometry.GetLocalSize().Y - AllottedGeometry.GetLocalSize().X / DesiredRatio) * .5f);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(LetterBoxSize, FSlateLayoutTransform(FVector2D(0.f, 0.f))),
				Brush,
				ESlateDrawEffect::None,
				Color
				);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(LetterBoxSize, FSlateLayoutTransform(FVector2D(0.f, AllottedGeometry.GetLocalSize().Y - LetterBoxSize.Y))),
				Brush,
				ESlateDrawEffect::None,
				Color
				);
		}
	}

	virtual TSharedPtr<SWidget> ConstructSettingsWidget() override
	{
		return SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				[
					WidgetHelpers::CreateSpinBox<float>(&Ratio1, 0.1f, 35)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(2.f, 0.f))
				[
					SNew(STextBlock)
					.Text(FText::FromString(TEXT(":")))
				]

				+ SHorizontalBox::Slot()
				[
					WidgetHelpers::CreateSpinBox<float>(&Ratio2, 0.1f, 35)
				]
			]

			+ SVerticalBox::Slot()
			.Padding(FMargin(0.f, 1.f, 0.f, 0.f))
			.AutoHeight()
			[
				WidgetHelpers::CreateColorWidget(&Color)
			];
	}

private:
	float Ratio1;
	float Ratio2;
	FLinearColor Color;
};

void SFilmOverlay::Construct(const FArguments& InArgs)
{
	FilmOverlays = InArgs._FilmOverlays;
}

int32 SFilmOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	if (!PrimaryFilmOverlay.IsNone())
	{
		if (const TSharedPtr<IFilmOverlay>* Found = UFilmOverlayToolkit::GetPrimaryFilmOverlays().Find(PrimaryFilmOverlay))
		{
			if (Found->IsValid())
			{
				(*Found)->Paint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
				++LayerId;
			}
		}
	}

	for (const TPair<FName, TSharedPtr<IFilmOverlay>>& Pair : UFilmOverlayToolkit::GetToggleableFilmOverlays())
	{
		if (Pair.Value.IsValid() && Pair.Value->IsEnabled())
		{
			Pair.Value->Paint(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			++LayerId;
		}
	}

	return LayerId;
}

// static
void SFilmOverlayOptions::RegisterDefaultOverlays()
{
	auto& ExistingPrimaryOverlays = UFilmOverlayToolkit::GetPrimaryFilmOverlays();

#define PREPARE_PRIMARY_OVERLAY(Name, Construct) { TSharedPtr<IFilmOverlay> Overlay = ExistingPrimaryOverlays.FindRef(Name); \
	if (!Overlay) { UFilmOverlayToolkit::RegisterPrimaryFilmOverlay(Name, Construct); }}

	PREPARE_PRIMARY_OVERLAY(NAME_None, MakeShared<FFilmOverlay_None>());
	PREPARE_PRIMARY_OVERLAY("Grid3x3", MakeShared<FFilmOverlay_Grid>(3, 3));
	PREPARE_PRIMARY_OVERLAY("Grid2x2", MakeShared<FFilmOverlay_Grid>(2, 2));
	PREPARE_PRIMARY_OVERLAY("Crosshair", MakeShared<FFilmOverlay_Crosshair>());
	PREPARE_PRIMARY_OVERLAY("Rabatment", MakeShared<FFilmOverlay_Rabatment>());

#undef PREPARE_PRIMARY_OVERLAY

	auto& ExistingToggleableOverlays = UFilmOverlayToolkit::GetToggleableFilmOverlays();

#define PREPARE_TOGGLEABLE_OVERLAY(Name, Construct) { TSharedPtr<IFilmOverlay> Overlay = ExistingToggleableOverlays.FindRef(Name); \
	if (!Overlay) { UFilmOverlayToolkit::RegisterToggleableFilmOverlay(Name, Construct); }}

	PREPARE_TOGGLEABLE_OVERLAY("ActionSafe", MakeShared<FFilmOverlay_SafeFrame>(LOCTEXT("ActionSafe", "Action Safe"), 95.f, FLinearColor::Red));
	PREPARE_TOGGLEABLE_OVERLAY("TitleSafe", MakeShared<FFilmOverlay_SafeFrame>(LOCTEXT("TitleSafe", "Title Safe"), 90.f, FLinearColor::Yellow));
	PREPARE_TOGGLEABLE_OVERLAY("CustomSafe", MakeShared<FFilmOverlay_SafeFrame>(LOCTEXT("CustomSafe", "Custom Safe"), 85.f, FLinearColor::Green));
	PREPARE_TOGGLEABLE_OVERLAY("LetterBox", MakeShared<FFilmOverlay_LetterBox>());

#undef PREPARE_TOGGLEABLE_OVERLAY
}

void SFilmOverlayOptions::Construct(const FArguments& InArgs, TSharedPtr<SFilmOverlay> InFilmOverlay)
{
	PrimaryColorTint = FLinearColor(1.f, 1.f, 1.f, .5f);

	RegisterDefaultOverlays();

	// used to choose between creating the whole button + menu widget, or just the menu widget
	// e.g. Old toolbar uses the combo button, while the new one does not
	const bool bIsComboButton = InArgs._IsComboButton.Get();

	TSharedPtr<SWidget> MainWidget;
	if (bIsComboButton)
	{
		// Styling replicates the same from the other old toolbar menu buttons
		MainWidget =
			SNew(SComboButton)
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("EditorViewportToolBar.Button"))
			.ForegroundColor(FSlateColor::UseStyle())
			.HasDownArrow(false)
			.ContentPadding(FMargin(0.0f, 0.0f))
			.OnGetMenuContent(this, &SFilmOverlayOptions::GetMenuContent)
			.ButtonContent()
			[
				SNew(SBox)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.WidthOverride(24)
				.HeightOverride(16)
				.ToolTipText(LOCTEXT("FilmOverlaysToolTip", "Displays a list of available film overlays to apply to this viewport."))
				[
					SNew(SImage)
					.Image(this, &SFilmOverlayOptions::GetCurrentThumbnail)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
	}
	else
	{
		MainWidget = GetMenuContent();
	}

	OverlayWidget = InFilmOverlay;
	if (TSharedPtr<SFilmOverlay> OverlayWidgetPinned = OverlayWidget.Pin())
	{
		if (IFilmOverlay* OldPrimaryOverlay = GetPrimaryFilmOverlay())
		{
			PrimaryColorTint = OldPrimaryOverlay->GetTint();
		}

		// CurrentPrimaryOverlay (and therefore Thumbnail) resets for non button version of this widget every time we create it, so let's keep it up to date
		if (!bIsComboButton)
		{
			FName WidgetPrimaryOverlay = OverlayWidgetPinned->GetPrimaryFilmOverlay();

			if (!WidgetPrimaryOverlay.IsNone())
			{
				SetPrimaryFilmOverlay(WidgetPrimaryOverlay);
			}
		}
	}

	ChildSlot
	[
		MainWidget.ToSharedRef()
	];
}

FLinearColor SFilmOverlayOptions::GetPrimaryColorTint() const
{
	return PrimaryColorTint;
}

void SFilmOverlayOptions::OnPrimaryColorTintChanged(const FLinearColor& Tint)
{
	IFilmOverlay* Overlay = GetPrimaryFilmOverlay();
	if (Overlay)
	{
		Overlay->SetTint(PrimaryColorTint);
	}
}

TSharedRef<SWidget> SFilmOverlayOptions::GetMenuContent()
{
	return SNew(SGridPanel)

		+ SGridPanel::Slot(0, 0)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 5.f)
			[
				SNew(SHeader)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("OverlaysHeader", "Composition Overlays"))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				ConstructPrimaryOverlaysMenu()
			]
		]

		+ SGridPanel::Slot(0, 1)
		.Padding(10)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.Padding(2.f, 0.f, 5.f, 0.f)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OverlayTint", "Tint: "))
			]

			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				WidgetHelpers::CreateColorWidget(&PrimaryColorTint, FOnColorPicked::CreateRaw(this, &SFilmOverlayOptions::OnPrimaryColorTintChanged))
			]
		]

		+ SGridPanel::Slot(1, 0)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 5.f)
			[
				SNew(SHeader)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SafeFrameHeader", "Frames"))
				]
			]
			
			+ SVerticalBox::Slot()
			[
				ConstructToggleableOverlaysMenu()
			]
		];
}

TSharedRef<SWidget> SFilmOverlayOptions::ConstructPrimaryOverlaysMenu()
{
	TSharedRef<SUniformGridPanel> OverlaysPanel = SNew(SUniformGridPanel).SlotPadding(10.f);

	TArray<FName> OverlayNames;
	UFilmOverlayToolkit::GetPrimaryFilmOverlays().GenerateKeyArray(OverlayNames);

	const int32 NumColumns = FMath::Log2(static_cast<float>(OverlayNames.Num() - 1));

	int32 ColumnIndex = 0, RowIndex = 0;
	for (int32 OverlayIndex = 0; OverlayIndex < OverlayNames.Num(); ++OverlayIndex)
	{
		IFilmOverlay& Overlay = *UFilmOverlayToolkit::GetPrimaryFilmOverlays()[OverlayNames[OverlayIndex]].Get();
		OverlaysPanel->AddSlot(ColumnIndex, RowIndex)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.OnClicked(this, &SFilmOverlayOptions::SetPrimaryFilmOverlay, OverlayNames[OverlayIndex])
			.ToolTipText(Overlay.GetToolTip())
			[
				SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.Padding(0.f, 4.f)
				.AutoHeight()
				.HAlign(HAlign_Center)
				[
					SNew(SBox)
					.WidthOverride(36)
					.HeightOverride(24)
					[
						SNew(SImage)
						.Image(Overlay.GetThumbnail())
					]
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Center)
				.Padding(0.f, 0.f, 0.f, 4.f)
				[
					SNew(STextBlock)
					.ColorAndOpacity(FAppStyle::GetColor("DefaultForeground"))
					.Text(Overlay.GetDisplayName())
				]
			]
		];

		if (++ColumnIndex >= NumColumns)
		{
			ColumnIndex = 0;
			++RowIndex;
		}
	}

	return OverlaysPanel;
}

TSharedRef<SWidget> SFilmOverlayOptions::ConstructToggleableOverlaysMenu()
{
	TSharedRef<SGridPanel> GridPanel = SNew(SGridPanel);

	int32 Row = 0;
	for (const TPair<FName, TSharedPtr<IFilmOverlay> >& Pair : UFilmOverlayToolkit::GetToggleableFilmOverlays())
	{
		TSharedPtr<IFilmOverlay> FilmOverlay = Pair.Value;
		if (!FilmOverlay.IsValid())
		{
			continue;
		}

		TSharedPtr<SWidget> Settings = FilmOverlay->ConstructSettingsWidget();
		if (!Settings.IsValid())
		{
			continue;		
		}

		auto OnCheckStateChanged = [=](ECheckBoxState InNewState){ FilmOverlay->SetEnabled(InNewState == ECheckBoxState::Checked); };
		auto IsChecked = [=]{ return FilmOverlay->IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; };
		auto IsEnabled = [=]{ return FilmOverlay->IsEnabled(); };

		Settings->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda(IsEnabled)));

		GridPanel->AddSlot(0, Row)
		.VAlign(VAlign_Center)
		.Padding(10.f, 5.f)
		[
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda(OnCheckStateChanged)
			.IsChecked_Lambda(IsChecked)
			.ToolTipText(FilmOverlay->GetToolTip())
			[
				SNew(STextBlock)
				.Text(Pair.Value->GetDisplayName())
			]
		];

		GridPanel->AddSlot(1, Row)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.Padding(10.f, 5.f)
		[
			SNew(SBox)
			.WidthOverride(100)
			[
				Settings.ToSharedRef()
			]
		];

		++Row;
	}

	return GridPanel;
}

FReply SFilmOverlayOptions::SetPrimaryFilmOverlay(FName InName)
{
	if (TSharedPtr<SFilmOverlay> OverlayWidgetPinned = OverlayWidget.Pin())
	{
		OverlayWidgetPinned->SetPrimaryFilmOverlay(InName);
	}

	IFilmOverlay* Overlay = GetPrimaryFilmOverlay();
	if (Overlay)
	{
		Overlay->SetTint(PrimaryColorTint);
	}
	return FReply::Unhandled();
}

FReply SFilmOverlayOptions::ToggleFilmOverlay(FName InName)
{
	const TSharedPtr<IFilmOverlay>* FilmOverlay = UFilmOverlayToolkit::GetToggleableFilmOverlays().Find(InName);

	if (FilmOverlay && FilmOverlay->IsValid())
	{
		FilmOverlay->Get()->SetEnabled(!FilmOverlay->Get()->IsEnabled());
	}
	return FReply::Unhandled();
}

TSharedPtr<SFilmOverlay> SFilmOverlayOptions::GetFilmOverlayWidget() const
{
	return OverlayWidget.Pin();
}

TArray<IFilmOverlay*> SFilmOverlayOptions::GetActiveFilmOverlays() const
{
	TArray<IFilmOverlay*> Overlays;

	if (IFilmOverlay* Overlay = GetPrimaryFilmOverlay())
	{
		Overlays.Add(Overlay);
	}

	for (const TPair<FName, TSharedPtr<IFilmOverlay> >& Pair : UFilmOverlayToolkit::GetToggleableFilmOverlays())
	{
		TSharedPtr<IFilmOverlay> FilmOverlay = Pair.Value;
		if (!FilmOverlay.IsValid())
		{
			continue;
		}

		if (FilmOverlay->IsEnabled())
		{
			Overlays.Add(FilmOverlay.Get());
		}
	}

	return Overlays;
}

const FSlateBrush* SFilmOverlayOptions::GetCurrentThumbnail() const
{
	TSharedPtr<SFilmOverlay> Widget = GetFilmOverlayWidget();
	if (!Widget.IsValid())
	{
		return FLevelSequenceEditorStyle::Get()->GetBrush("FilmOverlay.DefaultThumbnail");
	}

	FName PrimaryOverlay = Widget->GetPrimaryFilmOverlay();
	if (!PrimaryOverlay.IsNone())
	{
		if (const TSharedPtr<IFilmOverlay>* Found = UFilmOverlayToolkit::GetPrimaryFilmOverlays().Find(PrimaryOverlay))
		{
			if (Found->IsValid())
			{			
				return Found->Get()->GetThumbnail();
			}
		}
	}

	return FLevelSequenceEditorStyle::Get()->GetBrush("FilmOverlay.DefaultThumbnail");
}

IFilmOverlay* SFilmOverlayOptions::GetPrimaryFilmOverlay() const
{
	if (TSharedPtr<SFilmOverlay> FilmOverlayWidget = GetFilmOverlayWidget())
	{
		FName CurrentPrimaryOverlay = FilmOverlayWidget->GetPrimaryFilmOverlay();
		if (!CurrentPrimaryOverlay.IsNone())
		{
			return UFilmOverlayToolkit::GetPrimaryFilmOverlays()[CurrentPrimaryOverlay].Get();
		}
	}
	return nullptr;
}

void SFilmOverlayOptions::BindCommands(TSharedRef<FUICommandList> Bindings)
{
	const FCinematicViewportCommands& Commands = FCinematicViewportCommands::Get();

	TArray<FName> PrimaryOverlayNames;
	UFilmOverlayToolkit::GetPrimaryFilmOverlays().GenerateKeyArray(PrimaryOverlayNames);

	TArray<FName> ToggleableOverlayNames;
	UFilmOverlayToolkit::GetToggleableFilmOverlays().GenerateKeyArray(ToggleableOverlayNames);

	// Use a weak pointer so that if this widget is destroyed (e.g. when the submenu closes
	// but SCinematicLevelViewport's CommandList still holds these bindings), invoking the
	// shortcut does not crash via a dangling raw 'this' pointer.
	TWeakPtr<SFilmOverlayOptions> WeakThis = StaticCastSharedRef<SFilmOverlayOptions>(AsShared());

	Bindings->MapAction(
		Commands.Disabled,
		FExecuteAction::CreateLambda([WeakThis]
		{
			if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
			{
				Pinned->SetPrimaryFilmOverlay(NAME_None);
			}
		}));

	for (int32 OverlayIndex = 0; OverlayIndex < PrimaryOverlayNames.Num(); ++OverlayIndex)
	{
		if (Commands.Grid2x2.Get()->GetCommandName() == PrimaryOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.Grid2x2,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = PrimaryOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->SetPrimaryFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.Grid3x3.Get()->GetCommandName() == PrimaryOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.Grid3x3,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = PrimaryOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->SetPrimaryFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.Crosshair.Get()->GetCommandName() == PrimaryOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.Crosshair,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = PrimaryOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->SetPrimaryFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.Rabatment.Get()->GetCommandName() == PrimaryOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.Rabatment,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = PrimaryOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->SetPrimaryFilmOverlay(OverlayName);
					}
				}));
		}
	}

	for (int32 OverlayIndex = 0; OverlayIndex < ToggleableOverlayNames.Num(); ++OverlayIndex)
	{
		if (Commands.ActionSafe.Get()->GetCommandName() == ToggleableOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.ActionSafe,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = ToggleableOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->ToggleFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.TitleSafe.Get()->GetCommandName() == ToggleableOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.TitleSafe,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = ToggleableOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->ToggleFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.CustomSafe.Get()->GetCommandName() == ToggleableOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.CustomSafe,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = ToggleableOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->ToggleFilmOverlay(OverlayName);
					}
				}));
		}
		else if (Commands.Letterbox.Get()->GetCommandName() == ToggleableOverlayNames[OverlayIndex])
		{
			Bindings->MapAction(
				Commands.Letterbox,
				FExecuteAction::CreateLambda([WeakThis, OverlayName = ToggleableOverlayNames[OverlayIndex]]
				{
					if (TSharedPtr<SFilmOverlayOptions> Pinned = WeakThis.Pin())
					{
						Pinned->ToggleFilmOverlay(OverlayName);
					}
				}));
		}
	}
}

#undef LOCTEXT_NAMESPACE
