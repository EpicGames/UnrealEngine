// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/UICommandList.h"
#include "Widgets/SCompoundWidget.h"
#include "IFilmOverlay.h"

class FPaintArgs;
class FSlateWindowElementList;
struct FSlateBrush;


/** A widget that sits on top of a viewport, and draws custom content */
class SFilmOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFilmOverlay){}

		/** User provided array of overlays to draw */
		SLATE_ATTRIBUTE(TArray<IFilmOverlay*>, FilmOverlays)

	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs);

	/** Paint this widget */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	/** Assigns film overlays to this film overlay widget*/
	void SetFilmOverlays(const TAttribute<TArray<IFilmOverlay*>>& InFilmOverlays)
	{
		FilmOverlays = InFilmOverlays;
	}

	/** Sets the current primary film overlay */
	void SetPrimaryFilmOverlay(const FName InFilmOverlay)
	{
		PrimaryFilmOverlay = InFilmOverlay;
	}

	/** Get the current primary film overlay */
	FName GetPrimaryFilmOverlay() const
	{
		return PrimaryFilmOverlay;
	}

private:
	/** Attribute used once per frame to retrieve the film overlays to paint */
	TAttribute<TArray<IFilmOverlay*>> FilmOverlays;

	/** Currently selected primary film overlay */
	FName PrimaryFilmOverlay;
};

/** A custom widget that comprises a combo box displaying all available overlay options */
class SFilmOverlayOptions : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SFilmOverlayOptions)
		: _IsComboButton(true)
		{}
		/** Set this to false in order to directly show the menu content, instead of showing a button to summon it */
		SLATE_ATTRIBUTE(bool, IsComboButton)
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, TSharedPtr<SFilmOverlay> InFilmOverlay);

	/** Retrieve the actual overlay widget that this widget controls. Can be positioned in any other widget hierarchy. */
	TSharedPtr<SFilmOverlay> GetFilmOverlayWidget() const;

	/** Bind commands for the overlays */
	void BindCommands(TSharedRef<FUICommandList>);

	/** Register the default primary and toggleable overlays into UFilmOverlayToolkit (idempotent). */
	static void RegisterDefaultOverlays();

private:

	/** Generate menu content for the combo button */
	TSharedRef<SWidget> GetMenuContent();

	/** Construct the part of the menu that defines the set of film overlays */
	TSharedRef<SWidget> ConstructPrimaryOverlaysMenu();

	/** Construct the part of the menu that defines the set of toggleable overlays (currently just safe-frames) */
	TSharedRef<SWidget> ConstructToggleableOverlaysMenu();

private:

	/** Get the thumbnail to be displayed on the combo box button */
	const FSlateBrush* GetCurrentThumbnail() const;

	/** Get the primary film overlay ptr (may be nullptr) */
	IFilmOverlay* GetPrimaryFilmOverlay() const;

	/** Get an array of all enabled film overlays */
	TArray<IFilmOverlay*> GetActiveFilmOverlays() const;

	/** Set the current primary overlay to the specified name */
	FReply SetPrimaryFilmOverlay(FName InName);

	/** Get/Set the color tint override for the current primary overlay */
	FLinearColor GetPrimaryColorTint() const;
	void OnPrimaryColorTintChanged(const FLinearColor& Tint);
	
	/** Toggle the film overlay enabled or disabled */
	FReply ToggleFilmOverlay(FName InName);

private:
	/** Color tint to apply to primary overlays */
	FLinearColor PrimaryColorTint;

	/** The overlay widget we control - externally owned */
	TWeakPtr<SFilmOverlay> OverlayWidget;
};
