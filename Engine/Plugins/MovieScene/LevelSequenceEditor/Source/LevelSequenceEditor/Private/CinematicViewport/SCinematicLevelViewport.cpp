// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicViewport/SCinematicLevelViewport.h"
#include "AnimatedRange.h"
#include "EditorModeManager.h"
#include "Framework/Commands/UICommandList.h"
#include "Evaluation/MovieSceneSequenceTransform.h"
#include "Widgets/Layout/SBorder.h"
#include "LevelSequenceEditorToolkit.h"
#include "MovieScene.h"
#include "Rendering/SlateRenderer.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "LevelSequenceEditorCommands.h"
#include "SLevelViewport.h"
#include "ISequencer.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "SequencerKeyCollection.h"
#include "CinematicViewport/CinematicViewportCommands.h"
#include "CinematicViewport/SCinematicTransportRange.h"
#include "CinematicViewport/FilmOverlays.h"
#include "FilmOverlayToolkit.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Fonts/FontMeasure.h"
#include "AssetEditorViewportLayout.h"
#include "CinematicLevelViewportToolbarContext.h"
#include "LevelSequenceEditorStyle.h"
#include "Subsystems/PanelExtensionSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"

#define LOCTEXT_NAMESPACE "SCinematicLevelViewport"

namespace UE::CinematicViewport::Private
{
void AddOverlayOptionsEntry(UToolMenu* InMenu)
{
	if (!InMenu)
	{
		return;
	}

	UCinematicLevelViewportToolbarContext* const Context = InMenu->FindContext<UCinematicLevelViewportToolbarContext>();
	if (!Context)
	{
		return;
	}

	FToolMenuInsert InsertPosition("ViewportTypes", EToolMenuInsertType::After);
	FToolMenuSection& Section = InMenu->AddSection(NAME_None, FText(), InsertPosition);
	Section.AddSeparator(NAME_None);

	TWeakPtr<SCinematicLevelViewport> CinematicLevelViewport = Context->CinematicLevelViewport;
	TAttribute<FSlateIcon> IconAttribute = TAttribute<FSlateIcon>::CreateLambda(
		[WeakViewport = CinematicLevelViewport]()
		{
			if (TSharedPtr<SCinematicLevelViewport> CinematicViewportPinned = WeakViewport.Pin())
			{
				if (const TSharedPtr<SFilmOverlay>& OverlayWidget = CinematicViewportPinned->GetOverlayWidget())
				{
					FName CurrentPrimaryOverlay = OverlayWidget->GetPrimaryFilmOverlay();
					if (!CurrentPrimaryOverlay.IsNone())
					{
						// Combine strings to find the proper Icon
						FName BrushName = FName("FilmOverlay." + CurrentPrimaryOverlay.ToString() + ".Small");
						FSlateIcon Icon = FSlateIcon(FLevelSequenceEditorStyle::Get()->GetStyleSetName(), BrushName);
						if (Icon.IsSet())
						{
							return Icon;
						}
					}
				}
			}

			return FSlateIcon(FLevelSequenceEditorStyle::Get()->GetStyleSetName(), "FilmOverlay.Disabled.Small");
		}
	);

	FToolMenuEntry Entry = Section.AddSubMenu(
		"OverlayOptions",
		LOCTEXT("OverlayOptionsLabel", "Film Overlay Options"),
		LOCTEXT(
			"OverlayOptionsTooltip",
			"Displays a list of available film overlays to apply to this viewport."
		),
		FNewToolMenuDelegate::CreateLambda(
			[CinematicLevelViewport](UToolMenu* Submenu) -> void
			{
				if (TSharedPtr<SCinematicLevelViewport> CinematicLevelViewportPinned = CinematicLevelViewport.Pin())
				{
					TSharedRef<SFilmOverlayOptions> FilmOverlayOptions =
						SNew(SFilmOverlayOptions, CinematicLevelViewportPinned->GetOverlayWidget()).IsComboButton(false);

					Submenu->AddMenuEntry(
						"FilmOverlay", FToolMenuEntry::InitWidget("FilmOverlay", FilmOverlayOptions, FText())
					);
				}
			}
		),
		false,
		IconAttribute
	);

	Entry.SetShowInToolbarTopLevel(true);
	Section.AddEntry(Entry);
}
} // namespace UE::CinematicViewport::Private

template<typename T>
struct SNonThrottledSpinBox : SSpinBox<T>
{
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		FReply Reply = SSpinBox<T>::OnMouseButtonDown(MyGeometry, MouseEvent);
		if (Reply.IsEventHandled())
		{
			Reply.PreventThrottling();
		}
		return Reply;
	}
};

struct FTypeInterfaceProxy : INumericTypeInterface<double>
{
	TSharedPtr<INumericTypeInterface<double>> Impl;

	/** Gets the minimum and maximum fractional digits. */
	virtual int32 GetMinFractionalDigits() const override
	{
		return 0;
	}
	virtual int32 GetMaxFractionalDigits() const override
	{
		return 0;
	}

	/** Sets the minimum and maximum fractional digits - A minimum greater than 0 will always have that many trailing zeros */
	virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}

	virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& NewValue) override {}

	/** Convert the type to/from a string */
	virtual FString ToString(const double& Value) const override
	{
		if (Impl.IsValid())
		{
			return Impl->ToString(Value);
		}
		return FString();
	}

	virtual TOptional<double> FromString(const FString& InString, const double& InExistingValue) override
	{
		if (Impl.IsValid())
		{
			return Impl->FromString(InString, InExistingValue);
		}
		return TOptional<double>();
	}

	/** Check whether the typed character is valid */
	virtual bool IsCharacterValid(TCHAR InChar) const
	{
		if (Impl.IsValid())
		{
			return Impl->IsCharacterValid(InChar);
		}
		return false;
	}
};

FCinematicViewportClient::FCinematicViewportClient()
	: FLevelEditorViewportClient(nullptr)
{
	bDrawAxes = false;
	bIsRealtime = true;
	SetAllowCinematicControl(true);
	bDisableInput = false;
}

class SPreArrangedBox : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnArrange, const FGeometry&);

	SLATE_BEGIN_ARGS(SPreArrangedBox){}
		SLATE_EVENT(FOnArrange, OnArrange)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		OnArrange = InArgs._OnArrange;
		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override
	{
		OnArrange.ExecuteIfBound(AllottedGeometry);
		SCompoundWidget::OnArrangeChildren(AllottedGeometry, ArrangedChildren);
	}

private:

	FOnArrange OnArrange;
};

class SCinematicPreviewViewport : public SLevelViewport
{
public:
	virtual const FSlateBrush* OnGetViewportBorderBrush() const override { return nullptr; }
	virtual bool IsActorEditorContextVisible() const { return false; }
	virtual EVisibility GetSelectedActorsCurrentLevelTextVisibility() const override { return EVisibility::Collapsed; }

	virtual TSharedPtr<SWidget> BuildViewportToolbar() override
	{
		return nullptr;
	}

	// Used to add CinematicViewportContext to LevelViewport
	virtual void ExtendToolbarContext(FToolMenuContext& InToolMenuContext) override
	{
		UCinematicLevelViewportToolbarContext* CinematicViewportContext =
			NewObject<UCinematicLevelViewportToolbarContext>();

		CinematicViewportContext->CinematicLevelViewport = CinematicLevelViewport;

		InToolMenuContext.AddObject(CinematicViewportContext);
	}

	void SetCinematicLevelViewport(const TWeakPtr<SCinematicLevelViewport>& InCinematicLevelViewport)
	{
		CinematicLevelViewport = InCinematicLevelViewport;
	}

	static void ExtendLevelToolbar()
	{
		if (UToolMenu* const Submenu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar.Camera"))
		{
			Submenu->AddDynamicSection(
				"CinematicCameraExtensionDynamicSection",
				FNewToolMenuDelegate::CreateLambda(
					[](UToolMenu* InDynamicMenu)
					{
						UE::CinematicViewport::Private::AddOverlayOptionsEntry(InDynamicMenu);
					}
				)
			);
		}
	}

	TSharedPtr<SWidget> MakeExternalViewportToolbar()
	{
		// Adding Film Overlay Options widget to both new and old toolbars
		ExtendLevelToolbar();

		// clang-format off
		TSharedRef<SVerticalBox> ToolbarsWidget =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SLevelViewport::BuildViewportToolbar().ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.MaxHeight(1.0f)
			[
				CreateViewportIndicatorWidget(
					TAttribute<EVisibility>::CreateSP(this, &SCinematicPreviewViewport::OnGetCinematicViewportIndicatorVisibility)
					).ToSharedRef()
			];
		// clang-format on

		return ToolbarsWidget;
	}

	FSlateColor GetBorderColorAndOpacity() const
	{
		return OnGetViewportBorderColorAndOpacity();
	}

	const FSlateBrush* GetBorderBrush() const
	{
		return SLevelViewport::OnGetViewportBorderBrush();
	}

	EVisibility GetBorderVisibility() const
	{
		const EVisibility ViewportContentVisibility = SLevelViewport::OnGetViewportContentVisibility();
		return ViewportContentVisibility == EVisibility::Visible ? EVisibility::HitTestInvisible : ViewportContentVisibility;
	}

	virtual EVisibility OnGetFocusedViewportIndicatorVisibility() const override { return EVisibility::Collapsed; }

	EVisibility OnGetCinematicViewportIndicatorVisibility() const
	{
		return SLevelViewport::OnGetFocusedViewportIndicatorVisibility();
	}

private:
	bool bShowToolbar;

	TWeakPtr<SCinematicLevelViewport> CinematicLevelViewport;
};

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCinematicLevelViewport::Construct(const FArguments& InArgs)
{
	ParentLayout = InArgs._ParentLayout;
	LayoutName = InArgs._LayoutName;
	RevertToLayoutName = InArgs._RevertToLayoutName;

	ViewportClient = MakeShareable( new FCinematicViewportClient() );

	FAssetEditorViewportConstructionArgs ViewportConstructionArgs;
	ViewportConstructionArgs.ConfigKey = LayoutName;
	ViewportConstructionArgs.ParentLayout = ParentLayout.Pin();
	ViewportConstructionArgs.bRealtime = true;

	ViewportWidget = SNew(SCinematicPreviewViewport, ViewportConstructionArgs)
		.LevelEditorViewportClient(ViewportClient)
		.ParentLevelEditor(InArgs._ParentLevelEditor);

	ViewportWidget->SetCinematicLevelViewport(SharedThis(this));

	ViewportClient->SetViewportWidget(ViewportWidget);

	// Automatically engage game-view to hide editor only sprites. This needs to be done
	// after the Viewport Client and Widget are constructed as they reset the view to defaults
	// as part of their initialization.
	ViewportClient->SetGameView(true);

	TypeInterfaceProxy = MakeShareable( new FTypeInterfaceProxy );

	FLevelSequenceEditorToolkit::OnOpened().AddSP(this, &SCinematicLevelViewport::OnEditorOpened);

	FLinearColor Gray(.3f, .3f, .3f, 1.f);

	CommandList = MakeShared<FUICommandList>();

	// Ensure the commands are registered
	FCinematicViewportCommands::Register();
	FLevelSequenceEditorCommands::Register();

	OverlayWidget = SNew(SFilmOverlay).Visibility(EVisibility::HitTestInvisible);

	// Register default overlays and bind overlay shortcuts to persistent objects so the shortcuts
	// continue to work after toolbar refreshes (e.g. after Save All) destroy transient SFilmOverlayOptions.
	SFilmOverlayOptions::RegisterDefaultOverlays();
	{
		const FCinematicViewportCommands& Commands = FCinematicViewportCommands::Get();
		TWeakPtr<SFilmOverlay> WeakOverlay = OverlayWidget;

		auto MakePrimaryAction = [WeakOverlay](FName OverlayName) -> FExecuteAction
		{
			return FExecuteAction::CreateLambda([WeakOverlay, OverlayName]
			{
				if (TSharedPtr<SFilmOverlay> Pinned = WeakOverlay.Pin())
				{
					Pinned->SetPrimaryFilmOverlay(OverlayName);
				}
			});
		};

		auto MakeToggleAction = [](FName OverlayName) -> FExecuteAction
		{
			return FExecuteAction::CreateLambda([OverlayName]
			{
				if (const TSharedPtr<IFilmOverlay>* Overlay = UFilmOverlayToolkit::GetToggleableFilmOverlays().Find(OverlayName))
				{
					if (Overlay->IsValid())
					{
						(*Overlay)->SetEnabled(!(*Overlay)->IsEnabled());
					}
				}
			});
		};

		CommandList->MapAction(Commands.Disabled,    MakePrimaryAction(NAME_None));
		CommandList->MapAction(Commands.Grid3x3,     MakePrimaryAction(FName("Grid3x3")));
		CommandList->MapAction(Commands.Grid2x2,     MakePrimaryAction(FName("Grid2x2")));
		CommandList->MapAction(Commands.Crosshair,   MakePrimaryAction(FName("Crosshair")));
		CommandList->MapAction(Commands.Rabatment,   MakePrimaryAction(FName("Rabatment")));
		CommandList->MapAction(Commands.ActionSafe,  MakeToggleAction(FName("ActionSafe")));
		CommandList->MapAction(Commands.TitleSafe,   MakeToggleAction(FName("TitleSafe")));
		CommandList->MapAction(Commands.CustomSafe,  MakeToggleAction(FName("CustomSafe")));
		CommandList->MapAction(Commands.Letterbox,   MakeToggleAction(FName("LetterBox")));
	}

	DecoratedTransportControls = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(nullptr)
			.ForegroundColor(FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle()))
			[
				SNew(SNonThrottledSpinBox<double>)
				.TypeInterface(TypeInterfaceProxy)
				.Style(FAppStyle::Get(), "Sequencer.HyperlinkSpinBox")
				.Font(FAppStyle::GetFontStyle("Sequencer.FixedFont"))
				.OnValueCommitted(this, &SCinematicLevelViewport::OnTimeCommitted)
				.OnValueChanged(this, &SCinematicLevelViewport::SetTime)
				.MinValue(TOptional<double>())
				.MaxValue(TOptional<double>())
				.OnEndSliderMovement(this, &SCinematicLevelViewport::SetTime)
				.Value(this, &SCinematicLevelViewport::GetTime)
				.ToolTipText(LOCTEXT("TimeLocalToCurrentSequence", "The current time of the sequence relative to the focused sequence."))
				.Delta_Lambda([this]()
				{
					return UIData.OuterResolution.AsDecimal() * UIData.OuterPlayRate.AsInterval(); 
				})
				.LinearDeltaSensitivity(25)
				.MinDesiredWidth(this, &SCinematicLevelViewport::GetPlayTimeMinDesiredWidth)
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SAssignNew(TransportControlsContainer, SBox)
		]

		+ SHorizontalBox::Slot()
		[
			SNew(SSpacer)
		];

	// clang-format off
	TSharedRef<SWidget> MainViewport = 	SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("BlackBrush"))
		.ForegroundColor(Gray)
		.Padding(0)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([] { return GLevelEditorModeTools().IsViewportUIHidden() ? EVisibility::Hidden : EVisibility::Visible; })

				+ SHorizontalBox::Slot()
				[
					ViewportWidget->MakeExternalViewportToolbar().ToSharedRef()
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SSpacer)
					.Size(FVector2D(0, 55))
				]
			]

			+ SVerticalBox::Slot()
			[
				SNew(SPreArrangedBox)
				.OnArrange(this, &SCinematicLevelViewport::CacheDesiredViewportSize)
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					[
						SNew(SSpacer)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					[
						SNew(SBox)
						.HeightOverride(this, &SCinematicLevelViewport::GetDesiredViewportHeight)
						.WidthOverride(this, &SCinematicLevelViewport::GetDesiredViewportWidth)
						[
							SNew(SOverlay)

							+ SOverlay::Slot()
							[
								ViewportWidget.ToSharedRef()
							]

								 + SOverlay::Slot()[OverlayWidget.ToSharedRef()
							]
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.HAlign(HAlign_Center)
					.Padding(10.f, 0.f)
					[
						SAssignNew(ViewportControls, SBox)
						.Visibility(this, &SCinematicLevelViewport::GetControlsVisibility)
						.WidthOverride(this, &SCinematicLevelViewport::GetDesiredViewportWidth)
						.Padding(FMargin(0.f, 10.f, 0.f, 0.f))
						[
							SNew(SHorizontalBox)

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							[
								SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Left)
								.AutoWidth()
								[
									SNew(STextBlock)
									.ColorAndOpacity(Gray)
									.Text_Lambda([this]{ return UIData.ShotName; })
									.ToolTipText(LOCTEXT("CurrentSequence", "The name of the currently evaluated sequence."))
								]

								+ SHorizontalBox::Slot()
								.HAlign(HAlign_Right)
								.AutoWidth()
								.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
								[
									SNew(STextBlock)
									.ColorAndOpacity(Gray)
									.Text_Lambda([this] { return UIData.CameraName; })
									.ToolTipText(LOCTEXT("CurrentCamera", "The name of the current camera."))
								]
							]

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Center)
							.AutoWidth()
							[
								SNew(STextBlock)
								.ColorAndOpacity(Gray)
								.Text_Lambda([this] { return UIData.Filmback; })
								.ToolTipText(LOCTEXT("CurrentFilmback", "The name of the current shot's filmback (the imaging area of the frame/sensor)."))
							]

							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							[
								SNew(STextBlock)
								.Font(FAppStyle::GetFontStyle("Sequencer.FixedFont"))
								.ColorAndOpacity(Gray)
								.Text_Lambda([this] { return UIData.LocalPlaybackTime; })
								.ToolTipText(LOCTEXT("LocalPlaybackTime", "The current playback time relative to the currently evaluated sequence."))
								.MinDesiredWidth(this, &SCinematicLevelViewport::GetPlayTimeMinDesiredWidth)
							]
						]
					]
				
					+ SVerticalBox::Slot()
					[
						SNew(SSpacer)
					]
				]
			]

			+ SVerticalBox::Slot()
			.Padding(5.f)
			.AutoHeight()
			[
				SNew(SWidgetSwitcher)
				.WidgetIndex(this, &SCinematicLevelViewport::GetVisibleWidgetIndex)

				+ SWidgetSwitcher::Slot()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f)
					[
						SAssignNew(TransportRange, SCinematicTransportRange)
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(5.f, 0.f)
					[
						SAssignNew(TimeRangeContainer, SBox)
					]
				]

				+ SWidgetSwitcher::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SBox)
					.Padding(FMargin(5.f, 10.f))
					[
						SNew(STextBlock)
						.ColorAndOpacity(Gray)
						.Text(LOCTEXT("NoSequencerMessage", "No active Level Sequence Editor detected. Please edit a Level Sequence to enable full controls."))
					]
				]
			]
		];
	// clang-format on

	ChildSlot
	[
		SNew(SOverlay)

		+ SOverlay::Slot()
		[
			MainViewport
		]

		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage(ViewportWidget.Get(), &SCinematicPreviewViewport::GetBorderBrush)
			.BorderBackgroundColor(ViewportWidget.Get(), &SCinematicPreviewViewport::GetBorderColorAndOpacity)
			.Visibility(ViewportWidget.Get(), &SCinematicPreviewViewport::GetBorderVisibility)
			.Padding(0.0f)
			.ShowEffectWhenDisabled( false )
		]
	];

	FLevelSequenceEditorToolkit::IterateOpenToolkits([&](FLevelSequenceEditorToolkit& Toolkit){
		Setup(Toolkit);
		return false;
	});
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SLevelViewport> SCinematicLevelViewport::GetLevelViewport() const
{
	return ViewportWidget;
}

int32 SCinematicLevelViewport::GetVisibleWidgetIndex() const
{
	return CurrentToolkit.IsValid() ? 0 : 1;
}

EVisibility SCinematicLevelViewport::GetControlsVisibility() const
{
	return CurrentToolkit.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

TOptional<double> SCinematicLevelViewport::GetMinTime() const
{
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		FFrameRate   PlayRate      = Sequencer->GetLocalTime().Rate;
		UMovieScene* MovieScene    = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		double       TimeInSeconds = MovieScene->GetEditorData().WorkStart;

		return (TimeInSeconds*PlayRate).GetFrame().Value;
	}
	return TOptional<double>();
}

TOptional<double> SCinematicLevelViewport::GetMaxTime() const
{
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		FFrameRate   PlayRate      = Sequencer->GetLocalTime().Rate;
		UMovieScene* MovieScene    = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();
		double       TimeInSeconds = MovieScene->GetEditorData().WorkEnd;

		return (TimeInSeconds*PlayRate).GetFrame().Value;
	}
	return TOptional<double>();
}

void SCinematicLevelViewport::OnTimeCommitted(double Value, ETextCommit::Type)
{
	SetTime(Value);
}

void SCinematicLevelViewport::SetTime(double Value)
{
	// Clamp the value as the UI can't due to needing an unbounded spinbox for value-change-rate purposes.
	Value = FMath::Clamp(Value, GetMinTime().GetValue(), GetMaxTime().GetValue());

	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		FFrameRate SequencerPlayRate = Sequencer->GetLocalTime().Rate;
		Sequencer->SetLocalTime(FFrameTime::FromDecimal(Value));
	}
}

double SCinematicLevelViewport::GetTime() const
{
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		return Sequencer->GetLocalTime().Time.GetFrame().Value;
	}
	return 0;
}

float SCinematicLevelViewport::GetPlayTimeMinDesiredWidth() const
{
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		TRange<double> ViewRange = Sequencer->GetViewRange();

		FString LowerBoundStr = Sequencer->GetNumericTypeInterface()->ToString(ViewRange.GetLowerBoundValue());
		FString UpperBoundStr = Sequencer->GetNumericTypeInterface()->ToString(ViewRange.GetUpperBoundValue());

		// Always measure with the negative and subframe indicator so that the size doesn't change when there is and isn't a subframe
		if (!LowerBoundStr.Contains(TEXT("*")))
		{
			LowerBoundStr += TEXT("*");
		}
		if (!LowerBoundStr.Contains(TEXT("-")))
		{
			LowerBoundStr += TEXT("-");
		}
		if (!UpperBoundStr.Contains(TEXT("*")))
		{
			UpperBoundStr += TEXT("*");
		}
		if (!UpperBoundStr.Contains(TEXT("-")))
		{
			UpperBoundStr += TEXT("-");
		}

		const FSlateFontInfo PlayTimeFont = FAppStyle::GetFontStyle("Sequencer.FixedFont");

		const TSharedRef< FSlateFontMeasure > FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();

		FVector2D LowerTextSize = FontMeasureService->Measure(LowerBoundStr, PlayTimeFont);
		FVector2D UpperTextSize = FontMeasureService->Measure(UpperBoundStr, PlayTimeFont);

		return FMath::Max(LowerTextSize.X, UpperTextSize.X);
	}

	return 0.f;
}

void SCinematicLevelViewport::CacheDesiredViewportSize(const FGeometry& AllottedGeometry)
{
	FVector2D AllowableSpace = AllottedGeometry.GetLocalSize();
	AllowableSpace.Y -= ViewportControls->GetDesiredSize().Y;

	if (ViewportClient->IsAspectRatioConstrained() && ViewportClient->AspectRatio > 0)
	{
		const double MinSize = FMath::Min(AllowableSpace.X / (double)ViewportClient->AspectRatio, AllowableSpace.Y);
		DesiredViewportSize = FVector2D((double)ViewportClient->AspectRatio * MinSize, MinSize);
	}
	else
	{
		DesiredViewportSize = AllowableSpace;
	}
}

FOptionalSize SCinematicLevelViewport::GetDesiredViewportWidth() const
{
	return DesiredViewportSize.X;
}

FOptionalSize SCinematicLevelViewport::GetDesiredViewportHeight() const
{
	return DesiredViewportSize.Y;
}

FReply SCinematicLevelViewport::OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) 
{
	// Explicitly disallow the following keys which are by default mapped to navigate the sequencer timeline 
	// because we don't want viewport and timeline navigation at the same time. Viewport takes precedence. 
	if (InKeyEvent.GetKey() == EKeys::Up ||
		InKeyEvent.GetKey() == EKeys::Down ||
		InKeyEvent.GetKey() == EKeys::Left ||
		InKeyEvent.GetKey() == EKeys::Right ||
		InKeyEvent.GetKey() == EKeys::PageUp ||
		InKeyEvent.GetKey() == EKeys::PageDown)
	{
		return FReply::Unhandled();
	}

	if (CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	ISequencer* Sequencer = GetSequencer();
	if (Sequencer && Sequencer->GetCommandBindings()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SCinematicLevelViewport::Setup(FLevelSequenceEditorToolkit& NewToolkit)
{
	CurrentToolkit = StaticCastSharedRef<FLevelSequenceEditorToolkit>(NewToolkit.AsShared());

	NewToolkit.OnClosed().AddSP(this, &SCinematicLevelViewport::OnEditorClosed);

	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		TypeInterfaceProxy->Impl = Sequencer->GetNumericTypeInterface();

		if (TransportRange.IsValid())
		{
			TransportRange->SetSequencer(Sequencer->AsShared());
		}

		if (TransportControlsContainer.IsValid())
		{
			TransportControlsContainer->SetContent(Sequencer->MakeTransportControls(true));
		}

		if (TimeRangeContainer.IsValid())
		{
			const bool bShowWorkingRange = false, bShowViewRange = true, bShowPlaybackRange = true;
			TimeRangeContainer->SetContent(Sequencer->MakeTimeRange(DecoratedTransportControls.ToSharedRef(), bShowWorkingRange, bShowViewRange, bShowPlaybackRange));
		}
	}
}

void SCinematicLevelViewport::CleanUp()
{
	TransportControlsContainer->SetContent(SNullWidget::NullWidget);
	TimeRangeContainer->SetContent(SNullWidget::NullWidget);

	CurrentToolkit.Reset();
}

void SCinematicLevelViewport::OnEditorOpened(FLevelSequenceEditorToolkit& Toolkit)
{
	Setup(Toolkit);
}

void SCinematicLevelViewport::OnEditorClosed()
{
	CleanUp();

	FLevelSequenceEditorToolkit* NewToolkit = nullptr;
	FLevelSequenceEditorToolkit::IterateOpenToolkits([&](FLevelSequenceEditorToolkit& Toolkit){
		NewToolkit = &Toolkit;
		return false;
	});

	if (NewToolkit)
	{
		Setup(*NewToolkit);
	}
}

ISequencer* SCinematicLevelViewport::GetSequencer() const
{
	TSharedPtr<FLevelSequenceEditorToolkit> Toolkit = CurrentToolkit.Pin();
	if (Toolkit.IsValid())
	{
		return Toolkit->GetSequencer().Get();
	}

	return nullptr;
}

void SCinematicLevelViewport::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	ISequencer* Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	// Find the cinematic shot track
	UMovieSceneCinematicShotTrack* CinematicShotTrack = Cast<UMovieSceneCinematicShotTrack>(Sequence->GetMovieScene()->FindTrack(UMovieSceneCinematicShotTrack::StaticClass()));

	const FFrameRate OuterResolution = Sequencer->GetFocusedTickResolution();
	const FFrameRate OuterPlayRate   = Sequencer->GetFocusedDisplayRate();

	const FFrameTime OuterTime       = Sequencer->GetLocalTime().ConvertTo(OuterResolution);
	UIData.OuterResolution = OuterResolution;
	UIData.OuterPlayRate = OuterPlayRate;

	UMovieSceneCinematicShotSection* CinematicShotSection = nullptr;
	if (CinematicShotTrack)
	{
		for (UMovieSceneSection* Section : CinematicShotTrack->GetAllSections())
		{
			if (Section->GetRange().Contains(OuterTime.FrameNumber))
			{
				CinematicShotSection = CastChecked<UMovieSceneCinematicShotSection>(Section);
			}
		}
	}

	FText TimeFormat = LOCTEXT("TimeFormat", "{0}");

	TSharedPtr<INumericTypeInterface<double>> TimeDisplayFormatInterface = Sequencer->GetNumericTypeInterface();

	UMovieSceneSequence* SubSequence = CinematicShotSection ? CinematicShotSection->GetSequence() : nullptr;
	if (SubSequence)
	{
		FFrameRate                   InnerResolution       = SubSequence->GetMovieScene()->GetTickResolution();
		FMovieSceneSequenceTransform OuterToInnerTransform = CinematicShotSection ? CinematicShotSection->OuterToInnerTransform() : FMovieSceneSequenceTransform();
		const FFrameTime             InnerShotPosition	   = OuterTime * OuterToInnerTransform;

		UIData.LocalPlaybackTime = FText::Format(
			TimeFormat,
			FText::FromString(TimeDisplayFormatInterface->ToString(InnerShotPosition.GetFrame().Value))
		);

		if (CinematicShotSection)
		{
			UIData.ShotName = FText::FromString(CinematicShotSection->GetShotDisplayName());
		}
	}
	else
	{
		const FFrameTime DisplayTime = Sequencer->GetLocalTime().Time;

		UIData.LocalPlaybackTime = FText::Format(
			TimeFormat,
			FText::FromString(TimeDisplayFormatInterface->ToString(DisplayTime.GetFrame().Value))
			);

		UIData.ShotName = Sequence->GetDisplayName();
	}

	const FMovieSceneEditorData& EditorData = Sequence->GetMovieScene()->GetEditorData();

	FQualifiedFrameTime RootStartTime(EditorData.WorkStart * OuterPlayRate, OuterPlayRate);
	UIData.RootStartText = FText::Format(
		TimeFormat,
		FText::FromString(TimeDisplayFormatInterface->ToString(RootStartTime.Time.GetFrame().Value))
	);

	FQualifiedFrameTime RootEndTime(EditorData.WorkEnd * OuterPlayRate, OuterPlayRate);
	UIData.RootEndText = FText::Format(
		TimeFormat,
		FText::FromString(TimeDisplayFormatInterface->ToString(RootEndTime.Time.GetFrame().Value))
	);

	UIData.CameraName = FText::GetEmpty();

	UCameraComponent* CameraComponent = ViewportClient->GetCameraComponentForView();
	if (CameraComponent)
	{
		AActor* OuterActor = Cast<AActor>(CameraComponent->GetOuter());
		if (OuterActor != nullptr)
		{
			UIData.CameraName = FText::FromString(OuterActor->GetActorLabel());
		}

		UIData.Filmback = CameraComponent->GetFilmbackText();
	}
	else
	{
		UIData.Filmback = FText();
	}
}

#undef LOCTEXT_NAMESPACE
