// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "LevelEditorViewport.h"
#include "Subsystems/PanelExtensionSubsystem.h"

class SFilmOverlay;
class SFilmOverlayOptions;
namespace ETextCommit { enum Type : int; }
struct FOptionalSize;

class FLevelSequenceEditorToolkit;
class FAssetEditorViewportLayout;
class FUICommandList;
class ILevelEditor;
class ISequencer;
class SBox;
class SCinematicPreviewViewport;
class SCinematicTransportRange;
class SEditorViewport;
class SLevelViewport;

struct FQualifiedFrameTime;
struct FTypeInterfaceProxy;

/** Overridden level viewport client for this viewport */
struct FCinematicViewportClient : FLevelEditorViewportClient
{
	FCinematicViewportClient();

	void SetViewportWidget(const TSharedPtr<SEditorViewport>& InViewportWidget) { EditorViewportWidget = InViewportWidget; }
};


/** struct containing UI data - populated once per frame */
struct FUIData
{
	/** The name of the current shot */
	FText ShotName;
	/** The name of the current camera */
	FText CameraName;
	/** The name of the current shot's lens */
	FText Lens;
	/** The name of the current shot's filmback */
	FText Filmback;
	/** The text that represents the current playback time relative to the currently evaluated sequence. */
	FText LocalPlaybackTime;
	/** The text that represents the root start frame */
	FText RootStartText;
	/** The text that represents the root end frame */
	FText RootEndText;

	/** The tick resolution of the root */
	FFrameRate OuterResolution;
	/** The play rate of the root */
	FFrameRate OuterPlayRate;
};


class SCinematicLevelViewport : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCinematicLevelViewport) {}
		
		/** The unique name of this viewport inside its parent layout */
		SLATE_ARGUMENT(FName, LayoutName)
		/** Name of the viewport layout we should revert to at the user's request */
		SLATE_ARGUMENT(FName, RevertToLayoutName)
		/** Ptr to this viewport's parent layout */
		SLATE_ARGUMENT(TSharedPtr<FAssetEditorViewportLayout>, ParentLayout)
		/** Ptr to this viewport's parent level editor */
		SLATE_ARGUMENT(TWeakPtr<ILevelEditor>, ParentLevelEditor)

		SLATE_ARGUMENT(TWeakPtr<SCinematicPreviewViewport>, PreviewViewport)
	SLATE_END_ARGS()

	/** Access this viewport's viewport client */
	TSharedPtr<FCinematicViewportClient> GetViewportClient() const { return ViewportClient; }

	TSharedPtr<SLevelViewport> GetLevelViewport() const;

	/** Construct this widget */
	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

	const TSharedPtr<SFilmOverlay>& GetOverlayWidget()
	{
		return OverlayWidget;
	}

	TSharedPtr<FUICommandList> GetCommandList() { return CommandList; }
private:

	/** Set up this viewport to operate on the specified toolkit */
	void Setup(FLevelSequenceEditorToolkit& NewToolkit);

	/** Clean up this viewport after its toolkit has been closed */
	void CleanUp();

	/** Called when a level sequence editor toolkit has been opened */
	void OnEditorOpened(FLevelSequenceEditorToolkit& Toolkit);
	
	/** Called when the level sequence editor toolkit we were observing has been closed */
	void OnEditorClosed();

private:

	/** Get the sequencer ptr from our current toolkit, if set */
	ISequencer* GetSequencer() const;

private:

	/** Cache the desired viewport size based on the filled, allotted geometry */
	void CacheDesiredViewportSize(const FGeometry& AllottedGeometry);

	/** Get the desired width and height of the viewport */
	FOptionalSize GetDesiredViewportWidth() const;
	FOptionalSize GetDesiredViewportHeight() const;

	int32 GetVisibleWidgetIndex() const;
	EVisibility GetControlsVisibility() const;

private:

	TOptional<double> GetMinTime() const;
			  
	TOptional<double> GetMaxTime() const;

	void OnTimeCommitted(double Value, ETextCommit::Type);

	void SetTime(double Value);

	double GetTime() const;

	float GetPlayTimeMinDesiredWidth() const;

private:
	
	/** Widget where the scene viewport is drawn in */
	TSharedPtr<SCinematicPreviewViewport> ViewportWidget;

	/** The toolkit we're currently editing */
	TWeakPtr<FLevelSequenceEditorToolkit> CurrentToolkit;

	/** Commandlist used in the viewport (Maps commands to viewport specific actions) */
	TSharedPtr<FUICommandList> CommandList;

	/** Slot for transport controls */
	TSharedPtr<SBox> TransportControlsContainer, TimeRangeContainer;

	/** Decorated transport controls */
	TSharedPtr<SWidget> DecoratedTransportControls;

	/** Transport range widget */
	TSharedPtr<SCinematicTransportRange> TransportRange;
	
	/** Cached UI data */
	FUIData UIData;

	/** Cached desired size of the viewport */
	FVector2D DesiredViewportSize;
	
	/** Viewport controls widget that sits just below, and matches the width, of the viewport */
	TSharedPtr<SWidget> ViewportControls;

	/** Weak ptr to our parent layout */
	TWeakPtr<FAssetEditorViewportLayout> ParentLayout;

	/** Name of the viewport in the parent layout */
	FName LayoutName;
	/** Name of the viewport layout we should revert to at the user's request */
	FName RevertToLayoutName;

	/** The time spin box */
	TSharedPtr<FTypeInterfaceProxy> TypeInterfaceProxy;

	/** The level editor viewport client for this viewport */
	TSharedPtr<FCinematicViewportClient> ViewportClient;

	/** The overlay widget */
	TSharedPtr<SFilmOverlay> OverlayWidget;
};
