// Copyright Epic Games, Inc. All Rights Reserved.

#include "CinematicViewport/SCinematicTransportRange.h"
#include "MVVM/SharedList.h"
#include "SequencerKeyCollection.h"
#include "SequencerSettings.h"
#include "Misc/QualifiedFrameTime.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "Styling/AppStyle.h"
#include "Styles/LevelSequenceEditorStyle.h"
#include "ISequencer.h"
#include "TimeToPixel.h"
#include "TimeSliderArgs.h"

#define LOCTEXT_NAMESPACE "SCinematicTransportRange"

void SCinematicTransportRange::Construct(const FArguments& InArgs)
{
	bDraggingTime = false;
}

void SCinematicTransportRange::SetSequencer(TWeakPtr<ISequencer> InSequencer)
{
	WeakSequencer = InSequencer;
}

ISequencer* SCinematicTransportRange::GetSequencer() const
{
	return WeakSequencer.Pin().Get();
}

FVector2D SCinematicTransportRange::ComputeDesiredSize(float) const
{
	static const float MarkerHeight = 6.f;
	static const float TrackHeight = 8.f;
	return FVector2D(100.f, MarkerHeight + TrackHeight);
}

FReply SCinematicTransportRange::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bDraggingTime = true;
	SetTime(MyGeometry, MouseEvent);
		
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Scrubbing);
	}
	
	return FReply::Handled().CaptureMouse(AsShared()).PreventThrottling();
}

FReply SCinematicTransportRange::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bDraggingTime)
	{
		SetTime(MyGeometry, MouseEvent);
	}

	return FReply::Handled();
}

FReply SCinematicTransportRange::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bDraggingTime = false;

	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		Sequencer->SetPlaybackStatus(EMovieScenePlayerStatus::Stepping);
	}
	
	return FReply::Handled().ReleaseMouseCapture();
}

void SCinematicTransportRange::SetTime(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		float Lerp = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / MyGeometry.GetLocalSize().X;
		Lerp = FMath::Clamp(Lerp, 0.f, 1.f);

		FMovieSceneEditorData& EditorData = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData();
		double NewTimeSeconds = EditorData.ViewStart + (EditorData.ViewEnd - EditorData.ViewStart) * Lerp;

		FFrameRate TickResolution = Sequencer->GetFocusedTickResolution();
		FFrameTime ScrubTime = NewTimeSeconds * TickResolution;

		// Clamp first, snap to frame last
		TRange<FFrameNumber> PlayheadScrubbingRange = Sequencer->GetSequencerSettings()->GetPlayheadScrubbingRange(WeakSequencer);
		if (!PlayheadScrubbingRange.IsEmpty())
		{
			ScrubTime = UE::MovieScene::ClampToDiscreteRange(ScrubTime, PlayheadScrubbingRange);
		}

		ENearestKeyOption NearestKeyOption = ENearestKeyOption::NKO_None;

		TRange<double> ViewRange = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData().GetViewRange();

		TRange<FFrameNumber> VisibleFrameRange(
			(ViewRange.GetLowerBoundValue() * TickResolution).FloorToFrame(),
			(ViewRange.GetUpperBoundValue() * TickResolution).CeilToFrame()
		);

		TArrayView<const FFrameNumber> Keys = ActiveKeyCollection.IsValid() ? ActiveKeyCollection->GetKeysInRange(VisibleFrameRange, EFindKeyType::FKT_All) : TArrayView<const FFrameNumber>();

		if (Keys.Num() > 0)
		{
			if (Sequencer->GetSequencerSettings()->GetSnapPlayTimeToKeys() && (Sequencer->GetSequencerSettings()->GetIsSnapEnabled() || FSlateApplication::Get().GetModifierKeys().IsShiftDown()))
			{
				EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchKeys);
			}

			if (Sequencer->GetSequencerSettings()->GetSnapPlayTimeToSections() && (Sequencer->GetSequencerSettings()->GetIsSnapEnabled() || FSlateApplication::Get().GetModifierKeys().IsShiftDown()))
			{
				EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchSections);
			}
		}

		TArray<FMovieSceneMarkedFrame> MarkedFrames = Sequencer->GetMarkedFrames();

		if (MarkedFrames.Num() > 0)
		{
			if (Sequencer->GetSequencerSettings()->GetSnapPlayTimeToMarkers() && (Sequencer->GetSequencerSettings()->GetIsSnapEnabled() || FSlateApplication::Get().GetModifierKeys().IsShiftDown()))
			{
				EnumAddFlags(NearestKeyOption, ENearestKeyOption::NKO_SearchMarkers);
			}
		}

		if (NearestKeyOption != ENearestKeyOption::NKO_None)
		{
			FFrameTime NearestKey = Sequencer->OnGetNearestKey(ScrubTime, NearestKeyOption);

			FTimeToPixel TimeToPixelConverter(MyGeometry, ViewRange, Sequencer->GetFocusedTickResolution());

			const float ScrubPixel = TimeToPixelConverter.FrameToPixel(ScrubTime);
			const float NearestKeyPixel = TimeToPixelConverter.FrameToPixel(NearestKey);

			static float MouseTolerance = 20.f;

			if (FMath::IsNearlyEqual(ScrubPixel, NearestKeyPixel, MouseTolerance))
			{
				ScrubTime = NearestKey;
			}
		}
		
		Sequencer->SetLocalTime(ScrubTime, ESnapTimeMode::STM_None);
	}
}

void SCinematicTransportRange::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	bDraggingTime = false;
}

void SCinematicTransportRange::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	using namespace UE::Sequencer;

	ISequencer* Sequencer = GetSequencer();
	if (Sequencer)
	{
		TRange<double> ViewRange = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData().GetViewRange();

		// Anything within 3 pixel's worth of time is a duplicate as far as we're concerned
		FTimeToPixel TimeToPixelConverter(AllottedGeometry, ViewRange, Sequencer->GetFocusedTickResolution());
		const float DuplicateThreshold = (TimeToPixelConverter.PixelToSeconds(3.f) - TimeToPixelConverter.PixelToSeconds(0.f));

		Sequencer->GetKeysFromSelection(ActiveKeyCollection, FMath::Max(DuplicateThreshold, SMALL_NUMBER));
	}
}

void SCinematicTransportRange::DrawKeys(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bParentEnabled, const TArrayView<const FFrameNumber>& Keys, const TArrayView<const FLinearColor>& KeyColors, bool& bOutPlayMarkerOnKey) const
{
	ISequencer* Sequencer = GetSequencer();
	if (!Sequencer)
	{
		return;
	}

	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;

	static const float TrackOffsetY = 6.f;
	const float TrackHeight = AllottedGeometry.GetLocalSize().Y - TrackOffsetY;

	FFrameRate           TickResolution  = Sequencer->GetFocusedTickResolution();
	TRange<double>       ViewRange       = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData().GetViewRange();
	TRange<FFrameNumber> PlaybackRange   = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();

	const FFrameTime FramesPerPixel = (ViewRange.Size<double>() / AllottedGeometry.GetLocalSize().X) * TickResolution;
	const float FullRange = ViewRange.Size<float>();

	static const float BrushWidth = 7.f;
	static const float BrushHeight = 7.f;

	const float BrushOffsetY = TrackOffsetY + TrackHeight * .5f - BrushHeight * .5f;
	const FSlateBrush* KeyBrush = FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.CinematicViewportTransportRangeKey");
	const FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();

	FLinearColor KeyColor = KeyColors.Num() ? KeyColors[0] : FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

	int32 KeyIndex = 0;
	for (const FFrameNumber Time : Keys)
	{
		if (FMath::Abs(CurrentTime.Time - Time) < FramesPerPixel/2)
		{
			bOutPlayMarkerOnKey = true;
		}

		float Lerp = (Time/TickResolution - ViewRange.GetLowerBoundValue()) / FullRange;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId+2,
			AllottedGeometry.ToPaintGeometry(
				FVector2D(BrushWidth, BrushHeight),
				FSlateLayoutTransform(FVector2D(AllottedGeometry.GetLocalSize().X*Lerp - BrushWidth*.5f, BrushOffsetY))
			),
			KeyBrush,
			DrawEffects,
			KeyIndex < KeyColors.Num() ? KeyColors[KeyIndex] : KeyColor
		);

		++KeyIndex;
	}
}

int32 SCinematicTransportRange::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	using namespace UE::Sequencer;

	ISequencer* Sequencer = GetSequencer();

	const ESlateDrawEffect DrawEffects = bParentEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
	if (!Sequencer)
	{
		return LayerId;
	}

	static const float TrackOffsetY = 6.f;
	const float TrackHeight = AllottedGeometry.GetLocalSize().Y - TrackOffsetY;

	FFrameRate           TickResolution  = Sequencer->GetFocusedTickResolution();
	TRange<double>       ViewRange       = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetEditorData().GetViewRange();
	TRange<FFrameNumber> PlaybackRange   = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene()->GetPlaybackRange();

	// Anything within 3 pixel's worth of time is a duplicate as far as we're concerned
	FTimeToPixel TimeToPixelConverter(AllottedGeometry, ViewRange, TickResolution);
	const float DuplicateThreshold = (TimeToPixelConverter.PixelToSeconds(3.f) - TimeToPixelConverter.PixelToSeconds(0.f));

	const FFrameTime FramesPerPixel = (ViewRange.Size<double>() / AllottedGeometry.GetLocalSize().X) * TickResolution;

	FColor DarkGray(40, 40, 40);
	FColor MidGray(80, 80, 80);
	FColor LightGray(200, 200, 200);

	// Paint the left padding before the playback start
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		LayerId,
		AllottedGeometry.ToPaintGeometry(  FVector2D(AllottedGeometry.GetLocalSize().X, TrackHeight), FSlateLayoutTransform(FVector2f(0.f, TrackOffsetY))),
		FAppStyle::GetBrush("WhiteBrush"),
		DrawEffects,
		FLinearColor(DarkGray)
	);
	
	const float FullRange = ViewRange.Size<float>();

	const float PlaybackStartLerp	= (PlaybackRange.GetLowerBoundValue()/TickResolution - ViewRange.GetLowerBoundValue()) / FullRange;
	const float PlaybackEndLerp		= (PlaybackRange.GetUpperBoundValue()/TickResolution - ViewRange.GetLowerBoundValue()) / FullRange;

	// Draw the playback range
	FSlateDrawElement::MakeBox(
		OutDrawElements,
		++LayerId,
		AllottedGeometry.ToPaintGeometry(  FVector2f(AllottedGeometry.GetLocalSize().X*(PlaybackEndLerp - PlaybackStartLerp), TrackHeight), FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X*PlaybackStartLerp, TrackOffsetY))),
		FAppStyle::GetBrush("WhiteBrush"),
		DrawEffects,
		FLinearColor(MidGray)
	);

	const FQualifiedFrameTime CurrentTime = Sequencer->GetLocalTime();
	const float ProgressLerp = (CurrentTime.AsSeconds() - ViewRange.GetLowerBoundValue()) / FullRange;

	// Draw the playback progress
	if (ProgressLerp > PlaybackStartLerp)
	{
		const float ClampedProgressLerp = FMath::Clamp(ProgressLerp, PlaybackStartLerp, PlaybackEndLerp);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry( FVector2f(AllottedGeometry.GetLocalSize().X * (ClampedProgressLerp - PlaybackStartLerp), TrackHeight) , FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X*PlaybackStartLerp, TrackOffsetY))),
			FAppStyle::GetBrush("WhiteBrush"),
			DrawEffects,
			FLinearColor(LightGray)
		);
	}

	bool bPlayMarkerOnKey = false;

	const FLinearColor KeyframeColor = FAppStyle::GetSlateColor("SelectionColor").GetColor(FWidgetStyle());

	// Draw the current key collection tick marks
	if (ActiveKeyCollection.IsValid())
	{
		TRange<FFrameNumber> VisibleFrameRange(
			(ViewRange.GetLowerBoundValue() * TickResolution).FloorToFrame(),
			(ViewRange.GetUpperBoundValue() * TickResolution).CeilToFrame()
			);

		TArrayView<const FFrameNumber> Keys = ActiveKeyCollection->GetKeysInRange(VisibleFrameRange, EFindKeyType::FKT_All);

		DrawKeys(AllottedGeometry, OutDrawElements, LayerId, bParentEnabled, Keys, TArrayView<FLinearColor>(), bPlayMarkerOnKey);
	}

	// Draw the marked frames
	const FLinearColor DefaultMarkedFrameColor = Sequencer->GetSequencerSettings()->GetMarkedFrameColor();
	TArray<FMovieSceneMarkedFrame> MarkedFrames = Sequencer->GetMarkedFrames();
	if (MarkedFrames.Num())
	{
		int64 TotalMaxSeconds = static_cast<int64>(TNumericLimits<int32>::Max() / TickResolution.AsDecimal());

		FFrameNumber ThresholdFrames = (FMath::Max(DuplicateThreshold, SMALL_NUMBER) * TickResolution).FloorToFrame();
		if (ThresholdFrames.Value < -TotalMaxSeconds)
		{
			ThresholdFrames.Value = TotalMaxSeconds;
		}
		else if (ThresholdFrames.Value > TotalMaxSeconds)
		{
			ThresholdFrames.Value = TotalMaxSeconds;
		}

		TArray<FFrameNumber> GroupedTimes;
		TArray<FLinearColor> KeyColors;

		GroupedTimes.Reset(MarkedFrames.Num());
		KeyColors.Reset(MarkedFrames.Num());
		int32 Index = 0;
		while ( Index < MarkedFrames.Num() )
		{
			FFrameNumber PredicateTime = MarkedFrames[Index].FrameNumber;
			FLinearColor MarkedFrameColor = MarkedFrames[Index].bUseCustomColor ? MarkedFrames[Index].CustomColor : DefaultMarkedFrameColor;
			GroupedTimes.Add(PredicateTime);
			KeyColors.Add(MarkedFrameColor);
			KeyColors[KeyColors.Num()-1].A = 0.8f; // make the alpha consistent across all markers

			while (Index < MarkedFrames.Num() && FMath::Abs(MarkedFrames[Index].FrameNumber - PredicateTime) <= ThresholdFrames)
			{
				++Index;
			}
		}
		GroupedTimes.Shrink();
		KeyColors.Shrink();

		DrawKeys(AllottedGeometry, OutDrawElements, LayerId, bParentEnabled, GroupedTimes, KeyColors, bPlayMarkerOnKey);
	}

	// Draw the play marker
	{
		static const float BrushWidth = 11.f, BrushHeight = 6.f;
		const float PositionX = AllottedGeometry.GetLocalSize().X * ProgressLerp;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2f(BrushWidth, BrushHeight), FSlateLayoutTransform(FVector2f(PositionX - FMath::CeilToFloat(BrushWidth/2), 0.f))),
			FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.CinematicViewportPlayMarker"),
			DrawEffects,
			bPlayMarkerOnKey ? KeyframeColor : LightGray
		);

		if (!bPlayMarkerOnKey)
		{
			TArray<FVector2D> LinePoints;
			LinePoints.Add(FVector2D(PositionX, BrushHeight));
			LinePoints.Add(FVector2D(PositionX, AllottedGeometry.GetLocalSize().Y));

			FSlateDrawElement::MakeLines(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				DrawEffects,
				LightGray,
				false
			);
		}
	}

	// Draw the play bounds
	{
		static const float BrushWidth = 4.f;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId+1,
			AllottedGeometry.ToPaintGeometry(FVector2f(BrushWidth, TrackHeight), FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X*PlaybackStartLerp, TrackOffsetY))),
			FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.CinematicViewportRangeStart"),
			DrawEffects, 
			Sequencer->GetSequencerSettings()->GetPlaybackRangeStartColor().ToFColor(true)
		);

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId+1,
			AllottedGeometry.ToPaintGeometry(FVector2f(BrushWidth, TrackHeight), FSlateLayoutTransform(FVector2f(AllottedGeometry.GetLocalSize().X*PlaybackEndLerp - BrushWidth, TrackOffsetY))),
			FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.CinematicViewportRangeEnd"),
			DrawEffects,
			Sequencer->GetSequencerSettings()->GetPlaybackRangeEndColor().ToFColor(true)
		);
	}

	return LayerId;
}

#undef LOCTEXT_NAMESPACE
