// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

struct FCaptureLostEvent;

class FPaintArgs;
class FSlateWindowElementList;
class ISequencer;
class FSequencerKeyCollection;

class SCinematicTransportRange : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SCinematicTransportRange){}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs);

	/** Assign a new sequencer to this transport */
	void SetSequencer(TWeakPtr<ISequencer> InSequencer);

	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

private:

	void SetTime(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	ISequencer* GetSequencer() const;

	void DrawKeys(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, bool bParentEnabled, const TArrayView<const FFrameNumber>& Keys, const TArrayView<const FLinearColor>& KeyColors, bool& bOutPlayMarkerOnKey) const;

	/** The sequencer that we're controlling */
	TWeakPtr<ISequencer> WeakSequencer;

	/** The collection of keys for the currently active sequencer selection */
	TUniquePtr<FSequencerKeyCollection> ActiveKeyCollection;

	bool bDraggingTime;
};
