// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/LevelSequenceFactoryNew.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneToolsProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceFactoryNew)

#define LOCTEXT_NAMESPACE "MovieSceneFactory"


/* ULevelSequenceFactory structors
 *****************************************************************************/

ULevelSequenceFactoryNew::ULevelSequenceFactoryNew(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = ULevelSequence::StaticClass();
}


/* UFactory interface
 *****************************************************************************/

UObject* ULevelSequenceFactoryNew::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	auto NewLevelSequence = NewObject<ULevelSequence>(InParent, Name, Flags|RF_Transactional);
	NewLevelSequence->Initialize();

	// Set up some sensible defaults
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	FFrameRate TickResolution = NewLevelSequence->GetMovieScene()->GetTickResolution();
	NewLevelSequence->GetMovieScene()->SetPlaybackRange((ProjectSettings->DefaultStartTime*TickResolution).FloorToFrame(), (ProjectSettings->DefaultDuration*TickResolution).FloorToFrame().Value);

	return NewLevelSequence;
}


bool ULevelSequenceFactoryNew::ShouldShowInNewMenu() const
{
	return true;
}


#undef LOCTEXT_NAMESPACE

