// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilmOverlayToolkit.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilmOverlayToolkit)

TMap<FName, TSharedPtr<IFilmOverlay>> UFilmOverlayToolkit::PrimaryFilmOverlays;
TMap<FName, TSharedPtr<IFilmOverlay>> UFilmOverlayToolkit::ToggleableFilmOverlays;

void UFilmOverlayToolkit::RegisterPrimaryFilmOverlay(const FName& FilmOverlayName, TSharedPtr<IFilmOverlay> FilmOverlay)
{
	PrimaryFilmOverlays.Add(FilmOverlayName, FilmOverlay);
}

void UFilmOverlayToolkit::UnregisterPrimaryFilmOverlay(const FName& FilmOverlayName)
{
	PrimaryFilmOverlays.Remove(FilmOverlayName);
}

const TMap<FName, TSharedPtr<IFilmOverlay>>& UFilmOverlayToolkit::GetPrimaryFilmOverlays()
{
	return PrimaryFilmOverlays;
}

void UFilmOverlayToolkit::RegisterToggleableFilmOverlay(const FName& FilmOverlayName, TSharedPtr<IFilmOverlay> FilmOverlay)
{
	ToggleableFilmOverlays.Add(FilmOverlayName, FilmOverlay);
}

void UFilmOverlayToolkit::UnregisterToggleableFilmOverlay(const FName& FilmOverlayName)
{
	ToggleableFilmOverlays.Remove(FilmOverlayName);
}

const TMap<FName, TSharedPtr<IFilmOverlay>>& UFilmOverlayToolkit::GetToggleableFilmOverlays()
{
	return ToggleableFilmOverlays;
}
