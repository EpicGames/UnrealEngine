// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorBlueprintLibrary.h"

#include "Filters/ISequencerTrackFilters.h"
#include "ISequencer.h"
#include "MVVM/ViewModels/ChannelModel.h"
#include "MVVM/ViewModels/SectionModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "MVVM/SectionModelStorageExtension.h"
#include "MVVM/Selection/Selection.h"
#include "IKeyArea.h"
#include "LevelSequence.h"
#include "Channels/MovieSceneChannel.h"

#include "LevelEditorViewport.h"
#include "MovieSceneBindingProxy.h"
#include "Subsystems/AssetEditorSubsystem.h"

#include "ExtensionLibraries/MovieSceneSectionExtensions.h"

#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequencePlayer.h"

//For custom colors on channels, stored in editor pref's
#include "CurveEditorSettings.h"

#include "Sections/MovieSceneSubSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceEditorBlueprintLibrary)

namespace
{
	static TWeakPtr<ISequencer> CurrentSequencer;
}

bool ULevelSequenceEditorBlueprintLibrary::OpenLevelSequence(ULevelSequence* LevelSequence)
{
	if (LevelSequence)
	{
		return GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LevelSequence);
	}

	return false;
}

ULevelSequence* ULevelSequenceEditorBlueprintLibrary::GetCurrentLevelSequence()
{
	if (CurrentSequencer.IsValid())
	{
		return Cast<ULevelSequence>(CurrentSequencer.Pin()->GetRootMovieSceneSequence());
	}
	return nullptr;
}

ULevelSequence* ULevelSequenceEditorBlueprintLibrary::GetFocusedLevelSequence()
{
	if (CurrentSequencer.IsValid())
	{
		return Cast<ULevelSequence>(CurrentSequencer.Pin()->GetFocusedMovieSceneSequence());
	}
	return nullptr;
}

void ULevelSequenceEditorBlueprintLibrary::FocusLevelSequence(UMovieSceneSubSection* SubSection)
{
	if (CurrentSequencer.IsValid() && IsValid(SubSection))
	{
		CurrentSequencer.Pin()->FocusSequenceInstance(*SubSection);
	}
}

void ULevelSequenceEditorBlueprintLibrary::FocusParentSequence()
{
	if (CurrentSequencer.IsValid())
	{
		const TArray<FMovieSceneSequenceID>& Hierarchy = CurrentSequencer.Pin()->GetSubSequenceHierarchy();

		if (Hierarchy.Num() > 1)
		{
			// Copy the sequence ID since PopToSequenceInstance takes it by reference
			FMovieSceneSequenceID SequenceID = Hierarchy[Hierarchy.Num() -2];
			CurrentSequencer.Pin()->PopToSequenceInstance(SequenceID);
		}
	}
}

TArray<UMovieSceneSubSection*> ULevelSequenceEditorBlueprintLibrary::GetSubSequenceHierarchy()
{
	TArray<UMovieSceneSubSection*> SectionsHierarchy;
	
	if (CurrentSequencer.IsValid())
	{
		const TArray<FMovieSceneSequenceID>& Hierarchy = CurrentSequencer.Pin()->GetSubSequenceHierarchy();

		for (const FMovieSceneSequenceID Item : Hierarchy)
		{
			if (Item != MovieSceneSequenceID::Root)
			{
				// We return the whole hierarchy anyways, leaving it to the user to check for invalid pointers.
				SectionsHierarchy.Add(CurrentSequencer.Pin()->FindSubSection(Item));
			}
		}
	}
	return SectionsHierarchy;
}

void ULevelSequenceEditorBlueprintLibrary::CloseLevelSequence()
{
	if (CurrentSequencer.IsValid())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseAllEditorsForAsset(CurrentSequencer.Pin()->GetRootMovieSceneSequence());
	}
}

void ULevelSequenceEditorBlueprintLibrary::Play()
{
	const bool bTogglePlay = false;
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->OnPlay(bTogglePlay);
	}
}

void ULevelSequenceEditorBlueprintLibrary::Pause()
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->Pause();
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetCurrentTime(int32 NewFrame)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		FFrameTime GlobalTime = ConvertFrameTime(NewFrame, DisplayRate, TickResolution);
		if (GlobalTime == CurrentSequencer.Pin()->GetGlobalTime().Time)
		{
			CurrentSequencer.Pin()->ForceEvaluate();
		}
		else
		{
			CurrentSequencer.Pin()->SetGlobalTime(GlobalTime);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetGlobalPosition(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		UMovieSceneSequence* Sequence = CurrentSequencer.Pin()->GetFocusedMovieSceneSequence();

		FFrameTime Position = PlaybackParams.GetPlaybackPosition(Sequence);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Position = ConvertFrameTime(Position, DisplayRate, TickResolution);
		}

		if (Position == CurrentSequencer.Pin()->GetGlobalTime().Time)
		{
			CurrentSequencer.Pin()->ForceEvaluate();
		}
		else
		{
			CurrentSequencer.Pin()->SetGlobalTime(Position);
		}
	}
}

int32 ULevelSequenceEditorBlueprintLibrary::GetCurrentTime()
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		return ConvertFrameTime(CurrentSequencer.Pin()->GetGlobalTime().Time, TickResolution, DisplayRate).FloorToFrame().Value;
	}
	return 0;
}

FMovieSceneSequencePlaybackParams ULevelSequenceEditorBlueprintLibrary::GetGlobalPosition(EMovieSceneTimeUnit TimeUnit)
{
	FMovieSceneSequencePlaybackParams Params;
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		FQualifiedFrameTime GlobalTime = CurrentSequencer.Pin()->GetGlobalTime();

		Params.Frame = TimeUnit == EMovieSceneTimeUnit::DisplayRate ? ConvertFrameTime(GlobalTime.Time, TickResolution, DisplayRate) : GlobalTime.Time;
		Params.Timecode = GlobalTime.ToTimecode();
		Params.Time = GlobalTime.AsSeconds();
		
		return Params;
	}
	return Params;
}

void ULevelSequenceEditorBlueprintLibrary::SetCurrentLocalTime(int32 NewFrame)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		FFrameTime LocalTime = ConvertFrameTime(NewFrame, DisplayRate, TickResolution);
		if (LocalTime == CurrentSequencer.Pin()->GetLocalTime().Time)
		{
			CurrentSequencer.Pin()->ForceEvaluate();
		}
		else
		{
			CurrentSequencer.Pin()->SetLocalTime(LocalTime);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetLocalPosition(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		UMovieSceneSequence* Sequence = CurrentSequencer.Pin()->GetFocusedMovieSceneSequence();
		FFrameTime Position = PlaybackParams.GetPlaybackPosition(Sequence);
		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			Position = ConvertFrameTime(Position, DisplayRate, TickResolution);
		}

		if (Position == CurrentSequencer.Pin()->GetLocalTime().Time)
		{
			CurrentSequencer.Pin()->ForceEvaluate();
		}
		else
		{
			CurrentSequencer.Pin()->SetLocalTime(Position);
		}
	}
}

int32 ULevelSequenceEditorBlueprintLibrary::GetCurrentLocalTime()
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		return ConvertFrameTime(CurrentSequencer.Pin()->GetLocalTime().Time, TickResolution, DisplayRate).FloorToFrame().Value;
	}
	return 0;
}

FMovieSceneSequencePlaybackParams ULevelSequenceEditorBlueprintLibrary::GetLocalPosition(EMovieSceneTimeUnit TimeUnit)
{
	FMovieSceneSequencePlaybackParams Params;
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		FQualifiedFrameTime LocalTime = CurrentSequencer.Pin()->GetLocalTime();

		Params.Frame = TimeUnit == EMovieSceneTimeUnit::DisplayRate ? ConvertFrameTime(LocalTime.Time, TickResolution, DisplayRate) : LocalTime.Time;
		Params.Timecode = LocalTime.ToTimecode();
		Params.Time = LocalTime.AsSeconds();

		return Params;
	}
	return Params;
}

void ULevelSequenceEditorBlueprintLibrary::SetPlaybackSpeed(float NewPlaybackSpeed)
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->SetPlaybackSpeed(NewPlaybackSpeed);
	}
}

float ULevelSequenceEditorBlueprintLibrary::GetPlaybackSpeed()
{
	if (CurrentSequencer.IsValid())
	{
		return CurrentSequencer.Pin()->GetPlaybackSpeed();
	}

	return 0.f;
}

void ULevelSequenceEditorBlueprintLibrary::SetLoopMode(ESequencerLoopMode NewLoopMode)
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->GetSequencerSettings()->SetLoopMode(NewLoopMode);
	}
}

ESequencerLoopMode ULevelSequenceEditorBlueprintLibrary::GetLoopMode()
{
	if (CurrentSequencer.IsValid())
	{
		return CurrentSequencer.Pin()->GetSequencerSettings()->GetLoopMode();
	}

	return ESequencerLoopMode::SLM_NoLoop;
}

void ULevelSequenceEditorBlueprintLibrary::PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		if (TimeUnit == EMovieSceneTimeUnit::DisplayRate)
		{
			PlaybackParams.Frame = ConvertFrameTime(PlaybackParams.Frame, DisplayRate, TickResolution);
		}

		CurrentSequencer.Pin()->PlayTo(PlaybackParams);
	}
}

FMovieSceneSequencePlaybackParams ULevelSequenceEditorBlueprintLibrary::GetPlaybackStartPosition(EMovieSceneTimeUnit TimeUnit)
{
	FMovieSceneSequencePlaybackParams Params;

	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		UMovieSceneSequence* Sequence = CurrentSequencer.Pin()->GetFocusedMovieSceneSequence();
		TRange<FFrameNumber> PlaybackRange = Sequence->GetMovieScene()->GetPlaybackRange();
		FQualifiedFrameTime StartPosition(UE::MovieScene::DiscreteInclusiveLower(PlaybackRange), TickResolution);

		Params.Frame = TimeUnit == EMovieSceneTimeUnit::DisplayRate ? ConvertFrameTime(StartPosition.Time, TickResolution, DisplayRate) : StartPosition.Time;
		Params.Timecode = StartPosition.ToTimecode();
		Params.Time = StartPosition.AsSeconds();
	}

	return Params;
}

FMovieSceneSequencePlaybackParams ULevelSequenceEditorBlueprintLibrary::GetPlaybackEndPosition(EMovieSceneTimeUnit TimeUnit)
{
	FMovieSceneSequencePlaybackParams Params;

	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		UMovieSceneSequence* Sequence = CurrentSequencer.Pin()->GetFocusedMovieSceneSequence();
		TRange<FFrameNumber> PlaybackRange = Sequence->GetMovieScene()->GetPlaybackRange();

		FFrameTime OneFrame = FFrameRate::TransformTime(FFrameTime(1), DisplayRate, TickResolution);
		FQualifiedFrameTime EndPosition(UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange) - OneFrame, TickResolution);

		Params.Frame = TimeUnit == EMovieSceneTimeUnit::DisplayRate ? ConvertFrameTime(EndPosition.Time, TickResolution, DisplayRate) : EndPosition.Time;
		Params.Timecode = EndPosition.ToTimecode();
		Params.Time = EndPosition.AsSeconds();
	}

	return Params;
}

bool ULevelSequenceEditorBlueprintLibrary::IsPlaying()
{
	if (CurrentSequencer.IsValid())
	{
		return CurrentSequencer.Pin()->GetPlaybackStatus() == EMovieScenePlayerStatus::Playing;
	}
	return false;
}

TArray<UMovieSceneTrack*> ULevelSequenceEditorBlueprintLibrary::GetSelectedTracks()
{
	TArray<UMovieSceneTrack*> OutSelectedTracks;
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->GetSelectedTracks(OutSelectedTracks);
	}
	return OutSelectedTracks;
}

TArray<UMovieSceneSection*> ULevelSequenceEditorBlueprintLibrary::GetSelectedSections()
{
	TArray<UMovieSceneSection*> OutSelectedSections;
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->GetSelectedSections(OutSelectedSections);
	}
	return OutSelectedSections;
}

TArray<FSequencerChannelProxy> ULevelSequenceEditorBlueprintLibrary::GetSelectedChannels()
{
	TArray<FSequencerChannelProxy> OutSelectedChannels;
	if (CurrentSequencer.IsValid())
	{
		TArray<const IKeyArea*> SelectedKeyAreas;

		CurrentSequencer.Pin()->GetSelectedKeyAreas(SelectedKeyAreas);

		for (const IKeyArea* KeyArea : SelectedKeyAreas)
		{
			if (KeyArea)
			{
				FSequencerChannelProxy ChannelProxy(KeyArea->GetName(), KeyArea->GetOwningSection());
				OutSelectedChannels.Add(ChannelProxy);
			}
		}
	}
	return OutSelectedChannels;
}

TArray<FSequencerChannelProxy> ULevelSequenceEditorBlueprintLibrary::GetChannelsWithSelectedKeys()
{
	using namespace UE::Sequencer;

	TArray<FSequencerChannelProxy> OutSelectedChannels;
	TSet<FChannelModel*> ChannelModels;
	if (CurrentSequencer.IsValid())
	{
		const FKeySelection KeySelection = CurrentSequencer.Pin()->GetViewModel()->GetSelection()->KeySelection;

		for (FKeyHandle Key : KeySelection)
		{
			TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key);
			if (Channel)
			{
				ChannelModels.Add(Channel.Get());
			}
		}

		for (FChannelModel* Channel : ChannelModels)
		{
			if (Channel)
			{
				FSequencerChannelProxy ChannelProxy(Channel->GetChannelName(), Channel->GetSection());
				OutSelectedChannels.Add(ChannelProxy);
			}
		}
	}
	return OutSelectedChannels;
}

TArray<int32> ULevelSequenceEditorBlueprintLibrary::GetSelectedKeys(const FSequencerChannelProxy& ChannelProxy)
{
	TArray<int32> SelectedKeys;
	using namespace UE::Sequencer;

	if (CurrentSequencer.IsValid())
	{
		const FKeySelection KeySelection = CurrentSequencer.Pin()->GetViewModel()->GetSelection()->KeySelection;

		for (FKeyHandle Key : KeySelection)
		{
			if (TSharedPtr<FChannelModel> Channel = KeySelection.GetModelForKey(Key))
			{
				if (Channel->GetChannelName() == ChannelProxy.ChannelName)
				{
					int32 Index = Channel->GetChannel()->GetIndex(Key);
					if (Index != INDEX_NONE)
					{
						SelectedKeys.Add(Index);
					}
				}
			}
		}
	}
	return SelectedKeys;
}

TArray<UMovieSceneFolder*> ULevelSequenceEditorBlueprintLibrary::GetSelectedFolders()
{
	TArray<UMovieSceneFolder*> OutSelectedFolders;
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->GetSelectedFolders(OutSelectedFolders);
	}
	return OutSelectedFolders;
}

TArray<FMovieSceneBindingProxy> ULevelSequenceEditorBlueprintLibrary::GetSelectedBindings()
{
	TArray<FMovieSceneBindingProxy> OutSelectedBindings;
	if (CurrentSequencer.IsValid())
	{
		TArray<FGuid> OutSelectedGuids;
		CurrentSequencer.Pin()->GetSelectedObjects(OutSelectedGuids);

		UMovieSceneSequence* Sequence = CurrentSequencer.Pin()->GetFocusedMovieSceneSequence();

		for (const FGuid& SelectedGuid : OutSelectedGuids)
		{
			OutSelectedBindings.Add(FMovieSceneBindingProxy(SelectedGuid, Sequence));
		}
	}
	return OutSelectedBindings;
}

void ULevelSequenceEditorBlueprintLibrary::SelectTracks(const TArray<UMovieSceneTrack*>& Tracks)
{
	if (CurrentSequencer.IsValid())
	{
		for (UMovieSceneTrack* Track : Tracks)
		{
			CurrentSequencer.Pin()->SelectTrack(Track);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SelectSections(const TArray<UMovieSceneSection*>& Sections)
{
	if (CurrentSequencer.IsValid())
	{
		for (UMovieSceneSection* Section : Sections)
		{
			CurrentSequencer.Pin()->SelectSection(Section);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SelectChannels(const TArray<FSequencerChannelProxy>& Channels)
{
	if (CurrentSequencer.IsValid())
	{
		for (FSequencerChannelProxy ChannelProxy : Channels)
		{
			UMovieSceneSection* Section = ChannelProxy.Section;
			if (Section)
			{
				TArray<FName> ChannelNames;
				ChannelNames.Add(ChannelProxy.ChannelName);
				CurrentSequencer.Pin()->SelectByChannels(Section, ChannelNames, false, true);
			}
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SelectKeys(const FSequencerChannelProxy& ChannelProxy, const TArray<int32>& Indices)
{
	using namespace UE::Sequencer;
	if (CurrentSequencer.IsValid())
	{
		if (UMovieSceneSection* Section = ChannelProxy.Section)
		{
			FSectionModelStorageExtension* SectionModelStorage = CurrentSequencer.Pin()->GetViewModel()->GetRootModel()->CastDynamic<FSectionModelStorageExtension>();
			check(SectionModelStorage);

			TSharedPtr<FSectionModel> SectionHandle = SectionModelStorage->FindModelForSection(Section);
			if (SectionHandle)
			{
				TParentFirstChildIterator<FChannelGroupModel> KeyAreaNodes = SectionHandle->GetParentSectionOwnerModel().AsModel()->GetDescendantsOfType<FChannelGroupModel>();
				for (const TViewModelPtr<FChannelGroupModel>& KeyAreaNode : KeyAreaNodes)
				{
					if (KeyAreaNode->GetChannelName() == ChannelProxy.ChannelName)
					{
						if (TSharedPtr<FChannelModel> ChannelModel = KeyAreaNode->GetChannel(Section))
						{
							FMovieSceneChannel* MovieSceneChannel = ChannelModel->GetChannel();
							FKeySelection& KeySelection = CurrentSequencer.Pin()->GetViewModel()->GetSelection()->KeySelection;
							for (int32 Index : Indices)
							{
								if (Index >= 0 && Index < MovieSceneChannel->GetNumKeys())
								{
									FKeyHandle KeyHandle = MovieSceneChannel->GetHandle(Index);
									KeySelection.Select(ChannelModel, KeyHandle);
								}
							}
							break;
						}
					}
				}
			}
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SelectFolders(const TArray<UMovieSceneFolder*>& Folders)
{
	if (CurrentSequencer.IsValid())
	{
		for (UMovieSceneFolder* Folder : Folders)
		{
			CurrentSequencer.Pin()->SelectFolder(Folder);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::SelectBindings(const TArray<FMovieSceneBindingProxy>& ObjectBindings)
{
	if (CurrentSequencer.IsValid())
	{
		for (const FMovieSceneBindingProxy& ObjectBinding : ObjectBindings)
		{
			CurrentSequencer.Pin()->SelectObject(ObjectBinding.BindingID);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::DeselectBindings(const TArray<FMovieSceneBindingProxy>& ObjectBindings)
{
	if (CurrentSequencer.IsValid())
	{
		for (const FMovieSceneBindingProxy& ObjectBinding : ObjectBindings)
		{
			CurrentSequencer.Pin()->DeselectObject(ObjectBinding.BindingID);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::EmptySelection()
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->EmptySelection();
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetSelectionRangeStart(int32 NewFrame)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		CurrentSequencer.Pin()->SetSelectionRangeStart(ConvertFrameTime(NewFrame, DisplayRate, TickResolution));
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetSelectionRangeEnd(int32 NewFrame)
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		CurrentSequencer.Pin()->SetSelectionRangeEnd(ConvertFrameTime(NewFrame, DisplayRate, TickResolution));
	}
}

int32 ULevelSequenceEditorBlueprintLibrary::GetSelectionRangeStart()
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		return ConvertFrameTime(CurrentSequencer.Pin()->GetSelectionRange().GetLowerBoundValue(), TickResolution, DisplayRate).FloorToFrame().Value;
	}

	return 0;
}

int32 ULevelSequenceEditorBlueprintLibrary::GetSelectionRangeEnd()
{
	if (CurrentSequencer.IsValid())
	{
		FFrameRate DisplayRate = CurrentSequencer.Pin()->GetFocusedDisplayRate();
		FFrameRate TickResolution = CurrentSequencer.Pin()->GetFocusedTickResolution();

		return ConvertFrameTime(CurrentSequencer.Pin()->GetSelectionRange().GetUpperBoundValue(), TickResolution, DisplayRate).FloorToFrame().Value;
	}

	return 0;
}

void ULevelSequenceEditorBlueprintLibrary::SetSequencer(TSharedRef<ISequencer> InSequencer)
{
	CurrentSequencer = TWeakPtr<ISequencer>(InSequencer);
}

void ULevelSequenceEditorBlueprintLibrary::RefreshCurrentLevelSequence()
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
	}
}

void ULevelSequenceEditorBlueprintLibrary::ForceUpdate()
{
	if (CurrentSequencer.IsValid())
	{
		CurrentSequencer.Pin()->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::RefreshAllImmediately);
	}
}

TArray<UObject*> ULevelSequenceEditorBlueprintLibrary::GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding)
{
	TArray<UObject*> BoundObjects;
	if (CurrentSequencer.IsValid())
	{
		for (TWeakObjectPtr<> WeakObject : ObjectBinding.ResolveBoundObjects(CurrentSequencer.Pin()->GetFocusedTemplateID(), *CurrentSequencer.Pin()))
		{
			if (WeakObject.IsValid())
			{
				BoundObjects.Add(WeakObject.Get());
			}
		}

	}
	return BoundObjects;
}


bool ULevelSequenceEditorBlueprintLibrary::IsLevelSequenceLocked()
{
	if (CurrentSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin();
		UMovieSceneSequence* FocusedMovieSceneSequence = Sequencer->GetFocusedMovieSceneSequence();
		if (FocusedMovieSceneSequence) 
		{
			if (FocusedMovieSceneSequence->GetMovieScene()->IsReadOnly()) 
			{
				return true;
			}
			else
			{
				TArray<UMovieScene*> DescendantMovieScenes;
				MovieSceneHelpers::GetDescendantMovieScenes(Sequencer->GetFocusedMovieSceneSequence(), DescendantMovieScenes);

				for (UMovieScene* DescendantMovieScene : DescendantMovieScenes)
				{
					if (DescendantMovieScene && DescendantMovieScene->IsReadOnly())
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void ULevelSequenceEditorBlueprintLibrary::SetLockLevelSequence(bool bLock)
{
	if (CurrentSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin();

		if (Sequencer->GetFocusedMovieSceneSequence())
		{
			UMovieScene* MovieScene = Sequencer->GetFocusedMovieSceneSequence()->GetMovieScene();

			if (bLock != MovieScene->IsReadOnly()) 
			{
				MovieScene->Modify();
				MovieScene->SetReadOnly(bLock);
			}

			TArray<UMovieScene*> DescendantMovieScenes;
			MovieSceneHelpers::GetDescendantMovieScenes(Sequencer->GetFocusedMovieSceneSequence(), DescendantMovieScenes);

			for (UMovieScene* DescendantMovieScene : DescendantMovieScenes)
			{
				if (DescendantMovieScene && bLock != DescendantMovieScene->IsReadOnly())
				{
					DescendantMovieScene->Modify();
					DescendantMovieScene->SetReadOnly(bLock);
				}
			}

			Sequencer->NotifyMovieSceneDataChanged(EMovieSceneDataChangeType::Unknown);
		}
	}
}

bool ULevelSequenceEditorBlueprintLibrary::IsTrackFilterEnabled(const FText& TrackFilterName)
{
	return IsTrackFilterActive(TrackFilterName);
}

bool ULevelSequenceEditorBlueprintLibrary::IsTrackFilterActive(const FText& TrackFilterName)
{
	if (const TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin())
	{
		return Sequencer->GetFilterInterface()->IsFilterActiveByDisplayName(TrackFilterName.ToString());
	}
	return false;
}

void ULevelSequenceEditorBlueprintLibrary::SetTrackFilterEnabled(const FText& TrackFilterName, bool bEnabled)
{
	return SetTrackFilterActive(TrackFilterName, bEnabled);
}

void ULevelSequenceEditorBlueprintLibrary::SetTrackFilterActive(const FText& TrackFilterName, bool bActive)
{
	if (const TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin())
	{
		Sequencer->GetFilterInterface()->SetFilterActiveByDisplayName(TrackFilterName.ToString(), bActive);
	}
}

TArray<FText> ULevelSequenceEditorBlueprintLibrary::GetTrackFilterNames()
{
	if (const TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin())
	{
		return Sequencer->GetFilterInterface()->GetFilterDisplayNames();
	}
	return {};
}

bool ULevelSequenceEditorBlueprintLibrary::HasCustomColorForChannel(UClass* Class, const FString& Identifier)
{
	const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
	if (Settings)
	{
		TOptional<FLinearColor> OptColor = Settings->GetCustomColor(Class, Identifier);
		return OptColor.IsSet();
	}
	return false;
}

FLinearColor ULevelSequenceEditorBlueprintLibrary::GetCustomColorForChannel(UClass* Class, const FString& Identifier)
{
	FLinearColor Color(FColor::White);
	const UCurveEditorSettings* Settings = GetDefault<UCurveEditorSettings>();
	if (Settings)
	{
		TOptional<FLinearColor> OptColor = Settings->GetCustomColor(Class, Identifier);
		if (OptColor.IsSet())
		{
			return OptColor.GetValue();
		}
	}
	return Color;
}

void ULevelSequenceEditorBlueprintLibrary::SetCustomColorForChannel(UClass* Class, const FString& Identifier, const FLinearColor& NewColor)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		Settings->SetCustomColor(Class, Identifier, NewColor);
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetCustomColorForChannels(UClass* Class, const TArray<FString>& Identifiers, const TArray<FLinearColor>& NewColors)
{
	if (Identifiers.Num() != NewColors.Num())
	{
		return;
	}
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		for (int32 Index = 0; Index < Identifiers.Num(); ++Index)
		{
			const FString& Identifier = Identifiers[Index];
			const FLinearColor& NewColor = NewColors[Index];
			Settings->SetCustomColor(Class, Identifier, NewColor);
		}
	}
}

void ULevelSequenceEditorBlueprintLibrary::DeleteColorForChannels(UClass* Class, FString& Identifier)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		Settings->DeleteCustomColor(Class, Identifier);
	}
}

void ULevelSequenceEditorBlueprintLibrary::SetRandomColorForChannels(UClass* Class, const TArray<FString>& Identifiers)
{
	UCurveEditorSettings* Settings = GetMutableDefault<UCurveEditorSettings>();
	if (Settings)
	{
		for (int32 Index = 0; Index < Identifiers.Num(); ++Index)
		{
			const FString& Identifier = Identifiers[Index];
			FLinearColor NewColor = UCurveEditorSettings::GetNextRandomColor();
			Settings->SetCustomColor(Class, Identifier, NewColor);
		}
	}
}

bool ULevelSequenceEditorBlueprintLibrary::IsCameraCutLockedToViewport()
{
	if (CurrentSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin();
		return Sequencer->IsPerspectiveViewportCameraCutEnabled();
	}

	return false;
}

void ULevelSequenceEditorBlueprintLibrary::SetLockCameraCutToViewport(bool bLock)
{
	if (CurrentSequencer.IsValid())
	{
		TSharedPtr<ISequencer> Sequencer = CurrentSequencer.Pin();
		Sequencer->SetPerspectiveViewportCameraCutEnabled(bLock);
		Sequencer->ForceEvaluate();
	}
}


