// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorCommands.h"

#include "CineCameraActor.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "LevelSequenceEditorCommands"

FLevelSequenceEditorCommands::FLevelSequenceEditorCommands()
	: TCommands<FLevelSequenceEditorCommands>("LevelSequenceEditor", LOCTEXT("LevelSequenceEditor", "Level Sequence Editor"), NAME_None, "LevelSequenceEditorStyle")
{
}

void FLevelSequenceEditorCommands::RegisterCommands()
{
	UI_COMMAND(CreateNewLevelSequenceInLevel, "Add Level Sequence", "Create a new level sequence asset, and place an instance of it in this level", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateNewLevelSequenceWithShotsInLevel, "Add Level Sequence with Shots", "Create a new level sequence with shots asset, and place an instance of it in this level", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ToggleCinematicViewportCommand, "Cinematic Viewport", "A viewport layout tailored to cinematic preview", EUserInterfaceActionType::RadioButton, FInputChord());

	FUICommandInfo::MakeCommandInfo(
		AsShared(),
		ToggleCinematicViewportCommand,
		TEXT("Cinematic Viewport"),
		LOCTEXT("CinematicViewportLabel", "Cinematic Viewport"),
		LOCTEXT("CinematicViewport_Tooltip", "A viewport layout tailored to cinematic preview"),
		FSlateIconFinder::FindIconForClass(ACineCameraActor::StaticClass()),
		EUserInterfaceActionType::RadioButton,
		FInputChord()
	);

	UI_COMMAND(SnapSectionsToTimelineUsingSourceTimecode, "Snap Sections to Timeline using Source Timecode", "Snap selected sections to the position in the timeline matching their source timecode", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(SyncSectionsUsingSourceTimecode, "Sync Sections using Source Timecode", "Synchronize sections to the first selected section using source timecode", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(FixActorReferences, "Fix Actor References", "Try to automatically fix up broken actor bindings", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(AddActorsToBinding, "Add Selected", "Add selected actors to this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveActorsFromBinding, "Remove Selected", "Remove selected actors from this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReplaceBindingWithActors, "Replace with Selected", "Replace the object binding with selected actors", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveAllBindings, "Remove All", "Remove all bound actors from this track", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(RemoveInvalidBindings, "Remove Missing", "Remove missing objects bound to this track", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ImportFBX, "Import...", "Import the animation from an FBX file.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportFBX, "Export...", "Export the selected objects (all if none selected) and animation to an FBX file. (Shots and sub-scenes not supported)", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BakeTransform, "Bake Transform", "Bake transform in world space, removing any existing transform and attach tracks", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
