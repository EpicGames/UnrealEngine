// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FLevelSequenceEditorCommands
	: public TCommands<FLevelSequenceEditorCommands>
{
public:

	/** Default constructor. */
	FLevelSequenceEditorCommands();

	/** Initialize commands */
	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> CreateNewLevelSequenceInLevel;
	TSharedPtr<FUICommandInfo> CreateNewLevelSequenceWithShotsInLevel;
	TSharedPtr<FUICommandInfo> ToggleCinematicViewportCommand;
	
	TSharedPtr<FUICommandInfo> SnapSectionsToTimelineUsingSourceTimecode;
	TSharedPtr<FUICommandInfo> SyncSectionsUsingSourceTimecode;

	TSharedPtr<FUICommandInfo> FixActorReferences;

	TSharedPtr<FUICommandInfo> AddActorsToBinding;
	TSharedPtr<FUICommandInfo> RemoveActorsFromBinding;
	TSharedPtr<FUICommandInfo> ReplaceBindingWithActors;
	TSharedPtr<FUICommandInfo> RemoveAllBindings;
	TSharedPtr<FUICommandInfo> RemoveInvalidBindings;

	/** Imports animation from fbx. */
	TSharedPtr< FUICommandInfo > ImportFBX;

	/** Exports animation to fbx. */
	TSharedPtr< FUICommandInfo > ExportFBX;

	TSharedPtr<FUICommandInfo> BakeTransform;
};
