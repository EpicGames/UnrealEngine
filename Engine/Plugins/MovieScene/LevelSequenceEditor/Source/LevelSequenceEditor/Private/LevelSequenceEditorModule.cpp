// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorModule.h"

#include "AssetToolsModule.h"
#include "Blueprint/BlueprintSupport.h"
#include "BlueprintAssetHandler.h"
#include "Camera/CameraShakeSourceActor.h"
#include "CameraRig_Crane.h"
#include "CameraRig_Rail.h"
#include "CineCameraActor.h"
#include "CinematicViewport/CinematicViewportLayoutEntity.h"
#include "ContentBrowserMenuContexts.h"
#include "Factories/Factory.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "IAssetTools.h"
#include "ILevelSequenceModule.h"
#include "IPlacementModeModule.h"
#include "ISequencerModule.h"
#include "ISettingsModule.h"
#include "LevelEditor.h"
#include "LevelSequenceActor.h"
#include "LevelSequenceEditorCommands.h"
#include "Misc/LevelSequenceCustomization.h"
#include "Misc/LevelSequenceEditorActorBinding.h"
#include "Misc/LevelSequenceEditorActorSpawner.h"
#include "Misc/LevelSequenceEditorHelpers.h"
#include "Misc/LevelSequenceEditorSettings.h"
#include "Misc/MovieSceneSequenceEditor_LevelSequence.h"
#include "SequencerCommands.h"
#include "SequencerSettings.h"
#include "Styles/LevelSequenceEditorStyle.h"
#include "ToolMenus.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/SLevelSequenceFavoriteRating.h"

#define LOCTEXT_NAMESPACE "LevelSequenceEditor"


TSharedPtr<FLevelSequenceEditorStyle> FLevelSequenceEditorStyle::Singleton;

namespace UE::LevelSequenceEditorModule::Private
{
	/** Utility function to make a Menu entry with a custom Slate widget. */
	FNewToolMenuCustomWidget MakeCustomMenuWidgetDelegate(
		const TAttribute<FText>& Label, const TAttribute<FSlateIcon>& Icon, TSharedRef<SWidget> CustomWidget, EHorizontalAlignment CustomWidgetHAlign = HAlign_Fill)
	{
		return FNewToolMenuCustomWidget::CreateLambda([Label, Icon, CustomWidget, CustomWidgetHAlign](const FToolMenuContext& InContext, const FToolMenuCustomWidgetContext& WidgetContext)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(2, 0, 0, 0))
					.AutoWidth()
					[
						SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
							.Image(TAttribute<const FSlateBrush*>::CreateLambda([Icon]()
								{
									return Icon.IsSet() ? Icon.Get().GetIcon() : nullptr;
								}))
					]
					+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(8, 0, 0, 0))
						.AutoWidth()
						[
							SNew(STextBlock)
								.Text(Label)
								.TextStyle(WidgetContext.StyleSet, ISlateStyle::Join(WidgetContext.StyleName, ".Label"))
						]
						+ SHorizontalBox::Slot()
						.Padding(FMargin(8, 0, 0, 0))
						.VAlign(VAlign_Center)
						.HAlign(CustomWidgetHAlign)
						.FillWidth(1.0f)
						[
							CustomWidget
						];
			});
	};
}

/**
 * Implements the LevelSequenceEditor module.
 */
class FLevelSequenceEditorModule
	: public ILevelSequenceEditorModule, public FGCObject
{
public:

	FLevelSequenceEditorModule()
		: Settings(nullptr)
	{
	}

	// IModuleInterface interface

	virtual void StartupModule() override
	{
		FLevelSequenceEditorStyle::Get();

		RegisterEditorObjectBindings();
		RegisterEditorActorSpawner();
		RegisterMenuExtensions();
		RegisterLevelEditorExtensions();
		RegisterPlacementModeExtensions();
		RegisterSettings();
		RegisterSequenceEditor();
		RegisterSequenceCustomizations();

		class FLevelSequenceAssetBlueprintHandler : public IBlueprintAssetHandler
		{
			virtual UBlueprint* RetrieveBlueprint(UObject* InObject) const override
			{
				return CastChecked<ULevelSequence>(InObject)->GetDirectorBlueprint();
			}

			virtual bool AssetContainsBlueprint(const FAssetData& InAssetData) const
			{
				// Only have a blueprint if it contains the BlueprintPathWithinPackage tag
				return InAssetData.TagsAndValues.Contains(FBlueprintTags::BlueprintPathWithinPackage);
			}
		};

		FBlueprintAssetHandler::Get().RegisterHandler<FLevelSequenceAssetBlueprintHandler>(ULevelSequence::StaticClass()->GetClassPathName());
	}
	
	virtual void ShutdownModule() override
	{
		UnregisterEditorObjectBindings();
		UnregisterEditorActorSpawner();
		UnregisterMenuExtensions();
		UnregisterLevelEditorExtensions();
		UnregisterPlacementModeExtensions();
		UnregisterSettings();
		UnregisterSequenceEditor();
		UnregisterSequenceCustomizations();
	}

protected:

	/** Register sequencer editor object bindings */
	void RegisterEditorObjectBindings()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		ActorBindingDelegateHandle = SequencerModule.RegisterEditorObjectBinding(FOnCreateEditorObjectBinding::CreateStatic(&FLevelSequenceEditorModule::OnCreateActorBinding));
	}

	/** Register level sequence object spawner */
	void RegisterEditorActorSpawner()
	{
		ILevelSequenceModule& LevelSequenceModule = FModuleManager::LoadModuleChecked<ILevelSequenceModule>("LevelSequence");
		EditorActorSpawnerDelegateHandle = LevelSequenceModule.RegisterObjectSpawner(FOnCreateMovieSceneObjectSpawner::CreateStatic(&FLevelSequenceEditorActorSpawner::CreateObjectSpawner));
	}

	/** Registers level editor extensions. */
	void RegisterLevelEditorExtensions()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		FViewportTypeDefinition CinematicViewportType = FViewportTypeDefinition::FromType<FCinematicViewportLayoutEntity>(FLevelSequenceEditorCommands::Get().ToggleCinematicViewportCommand);
		LevelEditorModule.RegisterViewportType("Cinematic", CinematicViewportType);
	}

	/** Register menu extensions for the level editor toolbar. */
	void RegisterMenuExtensions()
	{
		FSequencerCommands::Register();
		FLevelSequenceEditorCommands::Register();

		const FLevelSequenceEditorCommands& Commands = FLevelSequenceEditorCommands::Get();

		CommandList = MakeShareable(new FUICommandList);
		CommandList->MapAction(Commands.CreateNewLevelSequenceInLevel,
			FExecuteAction::CreateStatic(&FLevelSequenceEditorModule::OnCreateActorInLevel)
		);
		CommandList->MapAction(Commands.CreateNewLevelSequenceWithShotsInLevel,
			FExecuteAction::CreateStatic(&FLevelSequenceEditorModule::OnCreateLevelSequenceWithShotsInLevel)
		);

		// Create and register the level editor toolbar menu extension
		CinematicsMenuExtender = MakeShareable(new FExtender);
		CinematicsMenuExtender->AddMenuExtension("LevelEditorNewCinematics", EExtensionHook::First, CommandList, FMenuExtensionDelegate::CreateStatic([](FMenuBuilder& MenuBuilder) {
			MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().CreateNewLevelSequenceInLevel);
		}));
		CinematicsMenuExtender->AddMenuExtension("LevelEditorNewCinematics", EExtensionHook::First, CommandList, FMenuExtensionDelegate::CreateStatic([](FMenuBuilder& MenuBuilder) {
			MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().CreateNewLevelSequenceWithShotsInLevel);
		}));

		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetAllLevelEditorToolbarCinematicsMenuExtenders().Add(CinematicsMenuExtender);

		// Create and register context menu extensions
		if (UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(ULevelSequence::StaticClass()))
		{
			const FName AssetActionSectionName = TEXT("GetAssetActions");
			FToolMenuSection& Section = Menu->FindOrAddSection(AssetActionSectionName);

			const FName FavoriteRatingMenuEntry = "LevelSequenceFavoriteRating";
			Section.AddDynamicEntry(FavoriteRatingMenuEntry, FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				if (UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					// Ensure only a single selected asset.
					const TArray<FAssetData>& SelectedAssets = Context->SelectedAssets;
					if (SelectedAssets.Num() == 1)
					{
						const FAssetData& LevelSequenceData = SelectedAssets[0];
						ULevelSequence* LevelSequence = Cast<ULevelSequence>(LevelSequenceData.GetAsset());

						FText SetFavoriteRatingLabel = LOCTEXT("SetFavoriteRating", "Set Rating");
						FSlateIcon SetFavoriteRatingIcon(FAppStyle::GetAppStyleSetName(), "Icons.Star");

						FToolMenuEntry RatingMenuEntry = FToolMenuEntry::InitMenuEntry(
							"SetFavoriteRating",
							SetFavoriteRatingLabel,
							LOCTEXT("SetFavoriteRatingTooltip", "Set Rating"),
							SetFavoriteRatingIcon,
							FUIAction()
						);

						TSharedRef<SLevelSequenceFavoriteRating> RatingWidget = SNew(SLevelSequenceFavoriteRating)
							.LevelSequence(LevelSequence);

						RatingMenuEntry.MakeCustomWidget = UE::LevelSequenceEditorModule::Private::MakeCustomMenuWidgetDelegate(
							SetFavoriteRatingLabel,
							SetFavoriteRatingIcon,
							RatingWidget,
							HAlign_Right
						);
						InSection.AddEntry(RatingMenuEntry);
					}
				}
			}));
		}
	}

	/** Registers placement mode extensions. */
	void RegisterPlacementModeExtensions()
	{
		FPlacementCategoryInfo Info(
			LOCTEXT("CinematicCategoryName", "Cinematic"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Cinematics"),
			"Cinematic",
			TEXT("PMCinematic"),
			25
		);
		Info.ShortDisplayName = LOCTEXT("CinematicShortCategoryName", "Cine");

		IPlacementModeModule::Get().RegisterPlacementCategory(Info);
		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable( new FPlaceableItem(nullptr, FAssetData(ACameraShakeSourceActor::StaticClass())) ));
		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable( new FPlaceableItem(nullptr, FAssetData(ACineCameraActor::StaticClass())) ));
		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable( new FPlaceableItem(nullptr, FAssetData(ACameraRig_Crane::StaticClass())) ));
		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable( new FPlaceableItem(nullptr, FAssetData(ACameraRig_Rail::StaticClass())) ));
		IPlacementModeModule::Get().RegisterPlaceableItem(Info.UniqueHandle, MakeShareable( new FPlaceableItem(nullptr, FAssetData(ALevelSequenceActor::StaticClass())) ));
	}

	/** Register settings objects. */
	void RegisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "LevelSequenceEditor",
				LOCTEXT("LevelSequenceEditorProjectSettingsName", "Level Sequence Editor"),
				LOCTEXT("LevelSequenceEditorProjectSettingsDescription", "Configure the Level Sequence Editor."),
				GetMutableDefault<ULevelSequenceEditorSettings>()
			);

			Settings = USequencerSettingsContainer::GetOrCreate<USequencerSettings>(TEXT("LevelSequenceEditor"));

			SettingsModule->RegisterSettings("Editor", "ContentEditors", "LevelSequenceEditor",
				LOCTEXT("LevelSequenceEditorSettingsName", "Level Sequence Editor"),
				LOCTEXT("LevelSequenceEditorSettingsDescription", "Configure the look and feel of the Level Sequence Editor."),
				Settings);	
		}
	}

	void RegisterSequenceEditor()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		SequenceEditorHandle = SequencerModule.RegisterSequenceEditor(ULevelSequence::StaticClass(), MakeUnique<FMovieSceneSequenceEditor_LevelSequence>());
	}

	void RegisterSequenceCustomizations()
	{
		ISequencerModule& SequencerModule = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer");
		SequencerModule.GetSequencerCustomizationManager()->RegisterInstancedSequencerCustomization(ULevelSequence::StaticClass(),
				FOnGetSequencerCustomizationInstance::CreateLambda([this]()
				{
					UE::Sequencer::FLevelSequenceCustomization* Customization = new UE::Sequencer::FLevelSequenceCustomization();
					TArray<FOnGetSequencerCustomizationInstance> CustomizationInstances;
					AdditionalLevelSequenceEditorCustomizations.GenerateValueArray(CustomizationInstances);
					for (const FOnGetSequencerCustomizationInstance& CustomizationInstance : CustomizationInstances)
					{
						Customization->AddCustomization(TUniquePtr<ISequencerCustomization>(CustomizationInstance.Execute()));
					}
					return Customization;
				}));
	}

protected:

	/** Unregisters sequencer editor object bindings */
	void UnregisterEditorActorSpawner()
	{
		ILevelSequenceModule* LevelSequenceModule = FModuleManager::GetModulePtr<ILevelSequenceModule>("LevelSequence");
		if (LevelSequenceModule)
		{
			LevelSequenceModule->UnregisterObjectSpawner(EditorActorSpawnerDelegateHandle);
		}
	}

	/** Unregisters sequencer editor object bindings */
	void UnregisterEditorObjectBindings()
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnRegisterEditorObjectBinding(ActorBindingDelegateHandle);
		}
	}

	/** Unregisters level editor extensions. */
	void UnregisterLevelEditorExtensions()
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
		if (LevelEditorModule)
		{
			LevelEditorModule->UnregisterViewportType("Cinematic");
		}
	}

	/** Unregisters menu extensions for the level editor toolbar. */
	void UnregisterMenuExtensions()
	{
		if (FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
		{
			LevelEditorModule->GetAllLevelEditorToolbarCinematicsMenuExtenders().Remove(CinematicsMenuExtender);
		}

		CinematicsMenuExtender = nullptr;
		CommandList = nullptr;

		FLevelSequenceEditorCommands::Unregister();
	}

	/** Unregisters placement mode extensions. */
	void UnregisterPlacementModeExtensions()
	{
		if (IPlacementModeModule::IsAvailable())
		{
			IPlacementModeModule::Get().UnregisterPlacementCategory("Cinematic");
		}
	}

	/** Unregister settings objects. */
	void UnregisterSettings()
	{
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "LevelSequenceEditor");

			SettingsModule->UnregisterSettings("Editor", "ContentEditors", "LevelSequenceEditor");
		}
	}

	void UnregisterSequenceEditor()
	{
		ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
		if (SequencerModule)
		{
			SequencerModule->UnregisterSequenceEditor(SequenceEditorHandle);
		}
	}

	void UnregisterSequenceCustomizations()
	{
		if (UObjectInitialized())
		{
			ISequencerModule* SequencerModule = FModuleManager::GetModulePtr<ISequencerModule>("Sequencer");
			if (SequencerModule)
			{
				SequencerModule->GetSequencerCustomizationManager()->UnregisterInstancedSequencerCustomization(ULevelSequence::StaticClass());
			}
		}
	}

protected:

	/** Callback for creating a new level sequence asset in the level. */
	static void OnCreateActorInLevel()
	{
		// Create a new level sequence
		IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();

		UObject* NewAsset = nullptr;

		// Attempt to create a new asset
		for (TObjectIterator<UClass> It ; It ; ++It)
		{
			UClass* CurrentClass = *It;
			if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
			{
				UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
				if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == ULevelSequence::StaticClass())
				{
					NewAsset = AssetTools.CreateAssetWithDialog(ULevelSequence::StaticClass(), Factory);
					break;
				}
			}
		}

		if (!NewAsset)
		{
			return;
		}

		// Spawn an actor at the origin, and either move infront of the camera or focus camera on it (depending on the viewport) and open for edit
		UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ALevelSequenceActor::StaticClass());
		if (!ensure(ActorFactory))
		{
			return;
		}

		AActor* Actor = GEditor->UseActorFactory(ActorFactory, FAssetData(NewAsset), &FTransform::Identity);
		if (Actor == nullptr)
		{
			return;
		}
		ALevelSequenceActor* NewActor = CastChecked<ALevelSequenceActor>(Actor);
		if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->IsPerspective())
		{
			GEditor->MoveActorInFrontOfCamera(*NewActor, GCurrentLevelEditingViewportClient->GetViewLocation(), GCurrentLevelEditingViewportClient->GetViewRotation().Vector());
		}
		else
		{
			GEditor->MoveViewportCamerasToActor(*NewActor, false);
		}

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(NewAsset);
	}

	/** Callback for creating a new level sequence asset in the level. */
	static void OnCreateLevelSequenceWithShotsInLevel()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		LevelSequenceEditorHelpers::OpenLevelSequenceWithShotsDialog(LevelEditorModule.GetLevelEditorTabManager().ToSharedRef());
	}

	FOnLevelSequenceWithShotsCreated& OnLevelSequenceWithShotsCreated() override
	{
		return OnLevelSequenceWithShotsCreatedEvent;
	}

	FAllowPlaybackContext& OnComputePlaybackContext() override
	{
		return OnComputePlaybackContextDelegate;
	}

	FGuid RegisterAdditionalLevelSequenceEditorCustomization(FOnGetSequencerCustomizationInstance OnGetInstance) override
	{
		FGuid Handle = FGuid::NewGuid();
		AdditionalLevelSequenceEditorCustomizations.Add(Handle, OnGetInstance);
		
		return Handle;
	}
	
	void UnregisterAdditionalLevelSequenceEditorCustomization(FGuid InHandle) override
	{
		AdditionalLevelSequenceEditorCustomizations.Remove(InHandle);
	}

	static TSharedRef<ISequencerEditorObjectBinding> OnCreateActorBinding(TSharedRef<ISequencer> InSequencer)
	{
		return MakeShareable(new FLevelSequenceEditorActorBinding(InSequencer));
	}

	/** FGCObject interface */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		if (Settings)
		{
			Collector.AddReferencedObject(Settings);
		}
	}

	virtual FString GetReferencerName() const override
	{
		return "FLevelSequenceEditorModule";
	}

private:

	/** Extender for the cinematics menu */
	TSharedPtr<FExtender> CinematicsMenuExtender;

	TSharedPtr<FUICommandList> CommandList;

	FOnLevelSequenceWithShotsCreated OnLevelSequenceWithShotsCreatedEvent;

	FAllowPlaybackContext OnComputePlaybackContextDelegate;

	FDelegateHandle ActorBindingDelegateHandle;

	FDelegateHandle EditorActorSpawnerDelegateHandle;

	TObjectPtr<USequencerSettings> Settings;

	FDelegateHandle SequenceEditorHandle;

	TMap<FGuid, FOnGetSequencerCustomizationInstance> AdditionalLevelSequenceEditorCustomizations;
};


IMPLEMENT_MODULE(FLevelSequenceEditorModule, LevelSequenceEditor);

#undef LOCTEXT_NAMESPACE
