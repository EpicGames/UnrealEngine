// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorToolkit.h"
#include "LevelSequence.h"
#include "Misc/LevelSequencePlaybackContext.h"
#include "Misc/LevelSequenceEditorMenuContext.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/ArrayBuilder.h"
#include "KeyParams.h"
#include "ILevelSequenceModule.h"
#include "LevelSequenceEditorModule.h"
#include "LevelSequenceEditorSubsystem.h"
#include "Misc/LevelSequenceEditorSettings.h"
#include "Misc/LevelSequenceEditorSpawnRegister.h"
#include "Misc/LevelSequenceEditorHelpers.h"
#include "LevelEditor.h"
#include "LevelEditorViewport.h"
#include "CineCameraActor.h"
#include "EditorModeManager.h"
#include "KeyPropertyParams.h"
#include "ISequencerModule.h"
#include "LevelSequenceActor.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Selection.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Sections/MovieSceneSubSection.h"
#include "ToolMenu.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "MovieSceneToolsProjectSettings.h"
#include "MovieSceneToolHelpers.h"
#include "SEditorViewport.h"
#include "ScopedTransaction.h"
#include "SequencerCommands.h"
#include "ToolMenuEntry.h"
#include "Widgets/Docking/SDockTab.h"
#include "SequencerSettings.h"
#include "LevelEditorSequencerIntegration.h"
#include "LevelSequenceEditorBlueprintLibrary.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "SequencerUtilities.h"
#include "ViewportToolbar/UnrealEdViewportToolbarContext.h"

// @todo sequencer: hack: setting defaults for transform tracks

#include "Tracks/MovieScene3DTransformTrack.h"

// To override Sequencer editor behavior for VR Editor 
#include "VREditorMode.h"
#include "VRModeSettings.h"


#define LOCTEXT_NAMESPACE "LevelSequenceEditor"

namespace UE::MovieScene::Private
{

const FName ViewportToolbarOwnerName = "LevelSequenceEditorViewportToolbar";

FToolMenuEntry CreateToggleViewportSelectionEntry(TWeakPtr<ISequencer> InWeakSequencer)
{
	return FToolMenuEntry::InitDynamicEntry(
		"DynamicToggleViewportSelection",
		FNewToolMenuSectionDelegate::CreateLambda(
			[InWeakSequencer](FToolMenuSection& InDynamicSection)
			{
				// First attempt to use the non-context Sequencer.
				TSharedPtr<ISequencer> Sequencer = InWeakSequencer.Pin();
				if (!Sequencer)
				{
					return;
				}

				if (!Sequencer->GetHostCapabilities().bSupportsViewportSelectability)
				{
					return;
				}

				const TSharedPtr<FUICommandList> CommandList =
					Sequencer->GetCommandBindings(ESequencerCommandBindings::Sequencer);

				FToolMenuEntry& Entry = InDynamicSection.AddMenuEntryWithCommandList(
					FSequencerCommands::Get().ToggleLimitViewportSelection, CommandList
				);
				Entry.InsertPosition.Position = EToolMenuInsertType::Last;
				Entry.SetShowInToolbarTopLevel(true);
			}
		)
	);
}

}; // namespace UE::MovieScene::Private

/* Local constants
 *****************************************************************************/

const FName FLevelSequenceEditorToolkit::SequencerMainTabId(TEXT("Sequencer_SequencerMain"));

namespace SequencerDefs
{
	static const FName SequencerAppIdentifier(TEXT("SequencerApp"));
}

static TArray<FLevelSequenceEditorToolkit*> OpenToolkits;

void FLevelSequenceEditorToolkit::IterateOpenToolkits(TFunctionRef<bool(FLevelSequenceEditorToolkit&)> Iter)
{
	for (FLevelSequenceEditorToolkit* Toolkit : OpenToolkits)
	{
		if (!Iter(*Toolkit))
		{
			return;
		}
	}
}

FLevelSequenceEditorToolkit::FLevelSequenceEditorToolkitOpened& FLevelSequenceEditorToolkit::OnOpened()
{
	static FLevelSequenceEditorToolkitOpened OnOpenedEvent;
	return OnOpenedEvent;
}

/* FLevelSequenceEditorToolkit structors
 *****************************************************************************/

FLevelSequenceEditorToolkit::FLevelSequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle)
	: LevelSequence(nullptr)
	, Style(InStyle)
{
	OpenToolkits.Add(this);
}


FLevelSequenceEditorToolkit::~FLevelSequenceEditorToolkit()
{
	OpenToolkits.Remove(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelEditor")))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		// @todo remove when world-centric mode is added
		LevelEditorModule.AttachSequencer(SNullWidget::NullWidget, nullptr);
		FLevelEditorSequencerIntegration::Get().RemoveSequencer(Sequencer.ToSharedRef());

		// unregister delegates

		LevelEditorModule.OnMapChanged().RemoveAll(this);
	}

	Sequencer->Close();

	

	if (FModuleManager::Get().IsModuleLoaded(TEXT("LevelSequenceEditor")))
	{
		auto& LevelSequenceEditorModule = FModuleManager::LoadModuleChecked<ILevelSequenceEditorModule>(TEXT("LevelSequenceEditor"));
		LevelSequenceEditorModule.OnLevelSequenceWithShotsCreated().RemoveAll(this);
	}
}


/* FLevelSequenceEditorToolkit interface
 *****************************************************************************/

void FLevelSequenceEditorToolkit::Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULevelSequence* InLevelSequence)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

	// Clear out the existing sequencer
	LevelEditorModule.AttachSequencer(nullptr, nullptr);

	// create tab layout
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_LevelSequenceEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
				->Split
				(
					FTabManager::NewStack()
						->AddTab(SequencerMainTabId, ETabState::OpenedTab)
				)
		);

	LevelSequence = InLevelSequence;
	PlaybackContext = MakeShared<FLevelSequencePlaybackContext>(InLevelSequence);

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = false;

	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, SequencerDefs::SequencerAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, LevelSequence);

	TSharedRef<FLevelSequenceEditorSpawnRegister> SpawnRegister = MakeShareable(new FLevelSequenceEditorSpawnRegister);

	// initialize sequencer
	FSequencerInitParams SequencerInitParams;
	{
		SequencerInitParams.RootSequence = LevelSequence;
		SequencerInitParams.bEditWithinLevelEditor = true;
		SequencerInitParams.ToolkitHost = InitToolkitHost;
		SequencerInitParams.SpawnRegister = SpawnRegister;

		SequencerInitParams.EventContexts.Bind(PlaybackContext.ToSharedRef(), &FLevelSequencePlaybackContext::GetEventContexts);
		SequencerInitParams.PlaybackContext.Bind(PlaybackContext.ToSharedRef(), &FLevelSequencePlaybackContext::GetPlaybackContextAsObject);
		SequencerInitParams.PlaybackClient.Bind(PlaybackContext.ToSharedRef(), &FLevelSequencePlaybackContext::GetPlaybackClientAsInterface);

		SequencerInitParams.ViewParams.UniqueName = "LevelSequenceEditor";
		SequencerInitParams.ViewParams.ScrubberStyle = ESequencerScrubberStyle::FrameBlock;
		SequencerInitParams.ViewParams.OnReceivedFocus.BindRaw(this, &FLevelSequenceEditorToolkit::OnSequencerReceivedFocus);
		SequencerInitParams.ViewParams.OnInitToolMenuContext.BindRaw(this, &FLevelSequenceEditorToolkit::OnInitToolMenuContext);

		SequencerInitParams.HostCapabilities.bSupportsCurveEditor = true;
		SequencerInitParams.HostCapabilities.bSupportsSaveMovieSceneAsset = true;
		SequencerInitParams.HostCapabilities.bSupportsRecording = true;
		SequencerInitParams.HostCapabilities.bSupportsRenderMovie = true;
		SequencerInitParams.HostCapabilities.bSupportsAddFromContentBrowser = true;
		SequencerInitParams.HostCapabilities.bSupportsSidebar = true;
		SequencerInitParams.HostCapabilities.bSupportsViewportSelectability = true;
	}

	ExtendSequencerToolbar("Sequencer.MainToolBar");

	Sequencer = FModuleManager::LoadModuleChecked<ISequencerModule>("Sequencer").CreateSequencer(SequencerInitParams);
	SpawnRegister->SetSequencer(Sequencer);
	Sequencer->OnActorAddedToSequencer().AddSP(this, &FLevelSequenceEditorToolkit::HandleActorAddedToSequencer);

	FLevelEditorSequencerIntegrationOptions Options;
	Options.bRequiresLevelEvents = true;
	Options.bRequiresActorEvents = true;

	FLevelEditorSequencerIntegration::Get().AddSequencer(Sequencer.ToSharedRef(), Options);
	ULevelSequenceEditorBlueprintLibrary::SetSequencer(Sequencer.ToSharedRef());

	// Now Attach so this window will apear in the correct front first order
	TSharedPtr<SDockTab> DockTab = LevelEditorModule.AttachSequencer(Sequencer->GetSequencerWidget(), SharedThis(this));
	if (DockTab.IsValid())
	{
		TAttribute<FText> LabelSuffix = TAttribute<FText>(this, &FLevelSequenceEditorToolkit::GetTabSuffix);
		DockTab->SetTabLabelSuffix(LabelSuffix);
	}

	// We need to find out when the user loads a new map, because we might need to re-create puppet actors
	// when previewing a MovieScene
	LevelEditorModule.OnMapChanged().AddRaw(this, &FLevelSequenceEditorToolkit::HandleMapChanged);

	ILevelSequenceEditorModule& LevelSequenceEditorModule = FModuleManager::LoadModuleChecked<ILevelSequenceEditorModule>("LevelSequenceEditor");
	LevelSequenceEditorModule.OnLevelSequenceWithShotsCreated().AddRaw(this, &FLevelSequenceEditorToolkit::HandleLevelSequenceWithShotsCreated);

	FLevelSequenceEditorToolkit::OnOpened().Broadcast(*this);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	{
		UWorld* World = PlaybackContext->GetPlaybackContext()->GetWorld();
		UVREditorMode* VRMode = Cast<UVREditorMode>( GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions( World )->FindExtension( UVREditorMode::StaticClass() ) );
		if (VRMode != nullptr)
		{
			VRMode->OnVREditingModeExit_Handler.BindSP(this, &FLevelSequenceEditorToolkit::HandleVREditorModeExit);
			USequencerSettings& SavedSequencerSettings = *Sequencer->GetSequencerSettings();
			VRMode->SaveSequencerSettings(Sequencer->GetKeyGroupMode() == EKeyGroupMode::KeyAll, Sequencer->GetAutoChangeMode(), SavedSequencerSettings);
			if (GetDefault<UVRModeSettings>()->bAutokeySequences)
			{
				// Override currently set auto-change behavior to always autokey
				Sequencer->SetAutoChangeMode(EAutoChangeMode::All);
				Sequencer->SetKeyGroupMode(EKeyGroupMode::KeyAll);
			}
			// Tell the VR Editor mode that Sequencer has refreshed
			VRMode->RefreshVREditorSequencer(Sequencer.Get());
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Extend the Level Editor viewport toolbar.
	{
		FToolMenuOwnerScoped ScopeOwner(UE::MovieScene::Private::ViewportToolbarOwnerName);

		const FName TransformMenu = "LevelEditor.ViewportToolbar.Transform";
		UToolMenu* const Menu = UToolMenus::Get()->ExtendMenu(TransformMenu);

		FToolMenuSection& SelectionSection = Menu->FindOrAddSection("Selection");

		{
			FToolMenuEntry Entry = UE::MovieScene::Private::CreateToggleViewportSelectionEntry(Sequencer.ToWeakPtr());
			Entry.InsertPosition.Position = EToolMenuInsertType::Last;

			SelectionSection.AddEntry(Entry);
		}
	}
}


/* IToolkit interface
 *****************************************************************************/

FText FLevelSequenceEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Level Sequence Editor");
}


FName FLevelSequenceEditorToolkit::GetToolkitFName() const
{
	static FName SequencerName("LevelSequenceEditor");
	return SequencerName;
}


FLinearColor FLevelSequenceEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}


FString FLevelSequenceEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Sequencer ").ToString();
}

FText FLevelSequenceEditorToolkit::GetTabSuffix() const
{
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();

	if (Sequence == nullptr)
	{
		return FText::GetEmpty();
	}
	
	const bool bIsDirty = Sequence->GetMovieScene()->GetOuter()->GetOutermost()->IsDirty();
	if (bIsDirty)
	{
		return LOCTEXT("TabSuffixAsterix", "*");
	}

	return FText::GetEmpty();
}

void FLevelSequenceEditorToolkit::BringToolkitToFront()
{
	ILevelSequenceEditorToolkit::BringToolkitToFront();

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
	LevelEditorTabManager->TryInvokeTab(LevelEditorTabIds::Sequencer);
}


/* FLevelSequenceEditorToolkit implementation
 *****************************************************************************/

void FLevelSequenceEditorToolkit::ExtendSequencerToolbar(FName InToolMenuName)
{
	FToolMenuOwnerScoped OwnerScoped(this);

	UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(InToolMenuName);

	const FToolMenuInsert SectionInsertLocation("BaseCommands", EToolMenuInsertType::Before);

	{
		ToolMenu->AddDynamicSection("LevelSequenceEditorDynamic", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{	
			ULevelSequenceEditorMenuContext* LevelSequenceEditorMenuContext = InMenu->FindContext<ULevelSequenceEditorMenuContext>();
			if (LevelSequenceEditorMenuContext && LevelSequenceEditorMenuContext->Toolkit.IsValid())
			{
				const FName SequencerToolbarStyleName = "SequencerToolbar";
			
				FToolMenuEntry PlaybackContextEntry = FToolMenuEntry::InitWidget(
					"PlaybackContext",
					LevelSequenceEditorMenuContext->Toolkit.Pin()->PlaybackContext->BuildWorldPickerCombo(),
					LOCTEXT("PlaybackContext", "PlaybackContext")
				);
				PlaybackContextEntry.StyleNameOverride = SequencerToolbarStyleName;

				FToolMenuSection& Section = InMenu->AddSection("LevelSequenceEditor");
				Section.AddEntry(PlaybackContextEntry);
			}
		}), SectionInsertLocation);
	}
}

void FLevelSequenceEditorToolkit::AddDefaultTracksForActor(AActor& Actor, const FGuid Binding)
{
	if (ULevelSequenceEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<ULevelSequenceEditorSubsystem>())
	{
		Subsystem->AddDefaultTracksForActor(Sequencer.ToSharedRef(), Actor, Binding);
	}
}


/* FLevelSequenceEditorToolkit callbacks
 *****************************************************************************/

void FLevelSequenceEditorToolkit::OnSequencerReceivedFocus()
{
	if (Sequencer.IsValid())
	{
		FLevelEditorSequencerIntegration::Get().OnSequencerReceivedFocus(Sequencer.ToSharedRef());
	}
}

void FLevelSequenceEditorToolkit::OnInitToolMenuContext(FToolMenuContext& MenuContext)
{
	ULevelSequenceEditorMenuContext* LevelSequenceEditorMenuContext = NewObject<ULevelSequenceEditorMenuContext>();
	LevelSequenceEditorMenuContext->Toolkit = SharedThis(this);
	MenuContext.AddObject(LevelSequenceEditorMenuContext);
}


void FLevelSequenceEditorToolkit::HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding)
{
	AddDefaultTracksForActor(*Actor, Binding);
}


void FLevelSequenceEditorToolkit::HandleVREditorModeExit()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UWorld* World = PlaybackContext->GetPlaybackContext()->GetWorld();
	UVREditorMode* VRMode = CastChecked<UVREditorMode>( GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions( World )->FindExtension( UVREditorMode::StaticClass() ) );

	// Reset sequencer settings
	Sequencer->SetAutoChangeMode(VRMode->GetSavedEditorState().AutoChangeMode);
	Sequencer->SetKeyGroupMode(VRMode->GetSavedEditorState().bKeyAllEnabled ? EKeyGroupMode::KeyAll : EKeyGroupMode::KeyChanged);
	VRMode->OnVREditingModeExit_Handler.Unbind();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLevelSequenceEditorToolkit::HandleMapChanged(class UWorld* NewWorld, EMapChangeType MapChangeType)
{
	// @todo sequencer: We should only wipe/respawn puppets that are affected by the world that is being changed! (multi-UWorld support)
	if( ( MapChangeType == EMapChangeType::LoadMap || MapChangeType == EMapChangeType::NewMap || MapChangeType == EMapChangeType::TearDownWorld) )
	{
		Sequencer->GetSpawnRegister().CleanUp(*Sequencer);
		CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
	}
}

void FLevelSequenceEditorToolkit::AddShot(UMovieSceneCinematicShotTrack* ShotTrack, const FString& ShotAssetName, const FString& ShotPackagePath, FFrameNumber ShotStartTime, FFrameNumber ShotEndTime, UObject* AssetToDuplicate, const FString& FirstShotAssetName)
{
	// Create a level sequence asset for the shot
	UObject* ShotAsset = LevelSequenceEditorHelpers::CreateLevelSequenceAsset(ShotAssetName, ShotPackagePath, AssetToDuplicate);
	UMovieSceneSequence* ShotSequence = Cast<UMovieSceneSequence>(ShotAsset);
	UMovieSceneSubSection* ShotSubSection = ShotTrack->AddSequence(ShotSequence, ShotStartTime, (ShotEndTime-ShotStartTime).Value);

	// Focus on the new shot
	GetSequencer()->ForceEvaluate();
	GetSequencer()->FocusSequenceInstance(*ShotSubSection);

	const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();
	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	// Create any subshots
	if (LevelSequenceSettings->SubSequenceNames.Num())
	{
		UMovieSceneSubTrack* SubTrack = Cast<UMovieSceneSubTrack>(ShotSequence->GetMovieScene()->FindTrack(UMovieSceneSubTrack::StaticClass()));
		if (!SubTrack)
		{
			SubTrack = Cast<UMovieSceneSubTrack>(ShotSequence->GetMovieScene()->AddTrack(UMovieSceneSubTrack::StaticClass()));
		}
	
		int32 RowIndex = 0;
		for (auto SubSequenceName : LevelSequenceSettings->SubSequenceNames)
		{
			FString SubSequenceAssetName = ShotAssetName + ProjectSettings->TakeSeparator + SubSequenceName.ToString();

			UMovieSceneSequence* SubSequence = nullptr;
			if (!LevelSequenceSettings->bInstanceSubSequences || ShotTrack->GetAllSections().Num() == 1)
			{
				UObject* SubSequenceAsset = LevelSequenceEditorHelpers::CreateLevelSequenceAsset(SubSequenceAssetName, ShotPackagePath);
				SubSequence = Cast<UMovieSceneSequence>(SubSequenceAsset);
			}
			else
			{
				// Get the corresponding sequence from the first shot
				UMovieSceneSubSection* FirstShotSubSection = Cast<UMovieSceneSubSection>(ShotTrack->GetAllSections()[0]);
				UMovieSceneSequence* FirstShotSequence = FirstShotSubSection->GetSequence();
				UMovieSceneSubTrack* FirstShotSubTrack = Cast<UMovieSceneSubTrack>(FirstShotSequence->GetMovieScene()->FindTrack(UMovieSceneSubTrack::StaticClass()));
			
				FString FirstShotSubSequenceAssetName = FirstShotAssetName + ProjectSettings->TakeSeparator + SubSequenceName.ToString();

				for (auto Section : FirstShotSubTrack->GetAllSections())
				{
					UMovieSceneSubSection* SubSection = Cast<UMovieSceneSubSection>(Section);
					if (SubSection->GetSequence()->GetDisplayName().ToString() == FirstShotSubSequenceAssetName)
					{
						SubSequence = SubSection->GetSequence();
						break;
					}
				}
			}

			if (SubSequence != nullptr)
			{
				UMovieSceneSubSection* SubSection = SubTrack->AddSequence(SubSequence, 0, (ShotEndTime-ShotStartTime).Value);
				SubSection->SetRowIndex(RowIndex++);
			}
		}
	}

	// Create a camera cut track with a camera if it doesn't already exist
	UMovieSceneTrack* CameraCutTrack = ShotSequence->GetMovieScene()->GetCameraCutTrack();
	if (!CameraCutTrack)
	{	
		// Create a cine camera asset
		ACineCameraActor* NewCamera = GCurrentLevelEditingViewportClient->GetWorld()->SpawnActor<ACineCameraActor>();
		NewCamera->SetActorLocation(GCurrentLevelEditingViewportClient->GetViewLocation(), false);
		NewCamera->SetActorRotation(GCurrentLevelEditingViewportClient->GetViewRotation());
		//pNewCamera->CameraComponent->FieldOfView = ViewportClient->ViewFOV; //@todo set the focal length from this field of view

		const USequencerSettings* SequencerSettings = GetDefault<USequencerSettings>();
		const bool bCreateSpawnableCamera = SequencerSettings->GetCreateSpawnableCameras() && ShotSequence->AllowsSpawnableObjects();

		FGuid CameraGuid;
		if (bCreateSpawnableCamera)
		{
			FString NewName = MovieSceneHelpers::MakeUniqueBindingName(ShotSequence->GetMovieScene(), FName::NameToDisplayString(ACineCameraActor::StaticClass()->GetFName().ToString(), false));
			UE::Sequencer::FCreateBindingParams CreateBindingParams;
			CreateBindingParams.bSpawnable = true;
			CreateBindingParams.BindingNameOverride = NewName;
			CameraGuid = FSequencerUtilities::CreateBinding(GetSequencer().ToSharedRef(), *NewCamera, CreateBindingParams);

			UObject* SpawnedCamera = GetSequencer()->FindSpawnedObjectOrTemplate(CameraGuid);
			if (SpawnedCamera)
			{
				GCurrentLevelEditingViewportClient->GetWorld()->EditorDestroyActor(NewCamera, true);
				NewCamera = Cast<ACineCameraActor>(SpawnedCamera);
			}
		}
		else
		{
			CameraGuid = GetSequencer()->CreateBinding(*NewCamera, NewCamera->GetActorLabel());
		}
		
		AddDefaultTracksForActor(*NewCamera, CameraGuid);
	}
}

void FLevelSequenceEditorToolkit::HandleLevelSequenceWithShotsCreated(UObject* LevelSequenceWithShotsAsset)
{
	UMovieSceneSequence* LevelSequenceWithShots = Cast<UMovieSceneSequence>(LevelSequenceWithShotsAsset);
	if (!LevelSequenceWithShots)
	{
		return;
	}

	const FScopedTransaction Transaction( LOCTEXT( "CreateLevelSequenceWithShots", "Create Level Sequence with Shots" ) );
	
	const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();
	uint32 NumShots = LevelSequenceSettings->NumShots;
	ULevelSequence* AssetToDuplicate = LevelSequenceSettings->SequenceToDuplicate.Get();

	const UMovieSceneToolsProjectSettings* ProjectSettings = GetDefault<UMovieSceneToolsProjectSettings>();

	UMovieSceneCinematicShotTrack* ShotTrack = LevelSequenceWithShots->GetMovieScene()->AddTrack<UMovieSceneCinematicShotTrack>();

	FFrameRate TickResolution = LevelSequenceWithShots->GetMovieScene()->GetTickResolution();

	// Create shots with a camera cut and a camera for each
	FFrameNumber SequenceStartTime = (ProjectSettings->DefaultStartTime * TickResolution).FloorToFrame();
	FFrameNumber ShotStartTime = SequenceStartTime;
	FFrameNumber ShotEndTime   = ShotStartTime;

	int32        ShotDuration  = (ProjectSettings->DefaultDuration * TickResolution).RoundToFrame().Value;
	
	if (ProjectSettings->DefaultDuration * NumShots > TickResolution.MaxSeconds())
	{
		ShotDuration = ((TickResolution.MaxSeconds() / NumShots) * TickResolution).RoundToFrame().Value;

		UE_LOGF(LogLevelSequenceEditor, Warning, "Default shot duration: %f is too large for creating %d shots. Clamping shot duration to: %f", ProjectSettings->DefaultDuration, NumShots, TickResolution.MaxSeconds() / NumShots);
	}
	
	FString FirstShotName; 
	for (uint32 ShotIndex = 0; ShotIndex < NumShots; ++ShotIndex)
	{
		ShotEndTime += ShotDuration;

		FString ShotName = MovieSceneToolHelpers::GenerateNewSubsequenceName(ShotTrack->GetAllSections(), ProjectSettings->ShotPrefix, ShotStartTime);
		FString ShotPackagePath = MovieSceneToolHelpers::GenerateNewSubsequencePath(LevelSequenceWithShots->GetMovieScene(), ProjectSettings->ShotPrefix, ShotName);

		if (ShotIndex == 0)
		{
			FirstShotName = ShotName;
		}

		AddShot(ShotTrack, ShotName, ShotPackagePath, ShotStartTime, ShotEndTime, AssetToDuplicate, FirstShotName);
		GetSequencer()->ResetToNewRootSequence(*LevelSequenceWithShots);

		ShotStartTime = ShotEndTime;
	}

	LevelSequenceWithShots->GetMovieScene()->SetPlaybackRange(SequenceStartTime, (ShotEndTime - SequenceStartTime).Value);

#if WITH_EDITORONLY_DATA
	const double SequenceStartSeconds = SequenceStartTime / TickResolution;
	const double SequenceEndSeconds   = ShotEndTime / TickResolution;
	const double OutputChange = (SequenceEndSeconds - SequenceStartSeconds) * 0.1;

	FMovieSceneEditorData& EditorData = LevelSequenceWithShots->GetMovieScene()->GetEditorData();
	EditorData.ViewStart = EditorData.WorkStart = SequenceStartSeconds - OutputChange;
	EditorData.ViewEnd   = EditorData.WorkEnd   = SequenceEndSeconds + OutputChange;
#endif

	GetSequencer()->ResetToNewRootSequence(*LevelSequenceWithShots);

	UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ALevelSequenceActor::StaticClass());
	if (!ensure(ActorFactory))
	{
		return;
	}

	if (ALevelSequenceActor* NewActor = Cast<ALevelSequenceActor>(GEditor->UseActorFactory(ActorFactory, FAssetData(LevelSequenceWithShotsAsset), &FTransform::Identity)))
	{
		if (GCurrentLevelEditingViewportClient != nullptr && GCurrentLevelEditingViewportClient->IsPerspective())
		{
			GEditor->MoveActorInFrontOfCamera(*NewActor, GCurrentLevelEditingViewportClient->GetViewLocation(), GCurrentLevelEditingViewportClient->GetViewRotation().Vector());
		}
		else
		{
			GEditor->MoveViewportCamerasToActor(*NewActor, false);
		}
	}
}

TSharedRef<SDockTab> FLevelSequenceEditorToolkit::HandleTabManagerSpawnTab(const FSpawnTabArgs& Args)
{
	TSharedPtr<SWidget> TabWidget = SNullWidget::NullWidget;

	if (Args.GetTabId() == SequencerMainTabId)
	{
		TabWidget = Sequencer->GetSequencerWidget();
	}

	return SNew(SDockTab)
		.Label(LOCTEXT("SequencerMainTitle", "Sequencer"))
		.TabColorScale(GetTabColorScale())
		.TabRole(ETabRole::PanelTab)
		[
			TabWidget.ToSharedRef()
		];
}

void FLevelSequenceEditorToolkit::SaveAsset_Execute()
{
	GetSequencer()->Save();
}

void FLevelSequenceEditorToolkit::OnClose()
{
	// Remove the viewport toolbar extensions we added earlier.
	UToolMenus::Get()->UnregisterOwnerByName(UE::MovieScene::Private::ViewportToolbarOwnerName);

	UWorld* World = PlaybackContext->GetPlaybackContext()->GetWorld();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UVREditorMode* VRMode = Cast<UVREditorMode>(GEditor->GetEditorWorldExtensionsManager()->GetEditorWorldExtensions(World)->FindExtension(UVREditorMode::StaticClass()));
	if (VRMode != nullptr)
	{
		// Null out the VR Mode's sequencer pointer
		VRMode->RefreshVREditorSequencer(nullptr);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	OpenToolkits.Remove(this);

	OnClosedEvent.Broadcast();
}

bool FLevelSequenceEditorToolkit::CanFindInContentBrowser() const
{
	// False so that sequencer doesn't take over Find In Content Browser functionality and always find the level sequence asset
	return false;
}

#undef LOCTEXT_NAMESPACE
