// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILevelSequenceEditorToolkit.h"

class FSpawnTabArgs;

struct FFrameNumber;

class AActor;
class FMenuBuilder;
class FToolBarBuilder;
class IAssetViewport;
class ISequencer;
class UActorComponent;
class UAnimInstance;
class ULevelSequence;
class UMovieSceneCinematicShotTrack;
class FLevelSequencePlaybackContext;
class UPrimitiveComponent;
enum class EMapChangeType : uint8;

/**
 * Implements an Editor toolkit for level sequences.
 */
class FLevelSequenceEditorToolkit
	: public ILevelSequenceEditorToolkit
	, public FGCObject
{ 
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InStyle The style set to use.
	 */
	FLevelSequenceEditorToolkit(const TSharedRef<ISlateStyle>& InStyle);

	/** Virtual destructor */
	virtual ~FLevelSequenceEditorToolkit();

public:

	/** Iterate all open level sequence editor toolkits */
	static void IterateOpenToolkits(TFunctionRef<bool(FLevelSequenceEditorToolkit&)> Iter);

	/** Called when the tab manager is changed */
	DECLARE_EVENT_OneParam(FLevelSequenceEditorToolkit, FLevelSequenceEditorToolkitOpened, FLevelSequenceEditorToolkit&);
	static FLevelSequenceEditorToolkitOpened& OnOpened();

	/** Called when the tab manager is changed */
	DECLARE_EVENT(FLevelSequenceEditorToolkit, FLevelSequenceEditorToolkitClosed);
	FLevelSequenceEditorToolkitClosed& OnClosed() { return OnClosedEvent; }

public:

	/**
	 * Initialize this asset editor.
	 *
	 * @param Mode Asset editing mode for this editor (standalone or world-centric).
	 * @param InitToolkitHost When Mode is WorldCentric, this is the level editor instance to spawn this editor within.
	 * @param LevelSequence The animation to edit.
	 * @param TrackEditorDelegates Delegates to call to create auto-key handlers for this sequencer.
	 */
	void Initialize(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, ULevelSequence* LevelSequence);

	/**
	 * Get the sequencer object being edited in this tool kit.
	 *
	 * @return Sequencer object.
	 */
	virtual TSharedPtr<ISequencer> GetSequencer() const override
	{
		return Sequencer;
	}

public:

	//~ FAssetEditorToolkit interface

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(LevelSequence);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FLevelSequenceEditorToolkit");
	}

	virtual void SaveAsset_Execute() override;
	virtual void OnClose() override;
	virtual bool CanFindInContentBrowser() const override;

public:

	//~ IToolkit interface

	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FText GetTabSuffix() const override;
	virtual void BringToolkitToFront() override;

protected:

	/** Add default movie scene tracks for the given actor. */
	void AddDefaultTracksForActor(AActor& Actor, const FGuid Binding);
	
	/** Add a shot to a sequence */
	void AddShot(UMovieSceneCinematicShotTrack* ShotTrack, const FString& ShotAssetName, const FString& ShotPackagePath, FFrameNumber ShotStartTime, FFrameNumber ShotEndTime, UObject* AssetToDuplicate, const FString& FirstShotAssetName);

	/** Called whenever sequencer has received focus */
	void OnSequencerReceivedFocus();

	/** Called whenever sequencer in initializing tool menu context */
	void OnInitToolMenuContext(FToolMenuContext& MenuContext);

private:

	void ExtendSequencerToolbar(FName InToolMenuName);

	/** Callback for map changes. */
	void HandleMapChanged(UWorld* NewWorld, EMapChangeType MapChangeType);

	/** Callback for when a sequence with shots is created. */
	void HandleLevelSequenceWithShotsCreated(UObject* LevelSequenceWithShotsAsset);

	/** Callback for spawning tabs. */
	TSharedRef<SDockTab> HandleTabManagerSpawnTab(const FSpawnTabArgs& Args);

	/** Callback for actor added to sequencer. */
	void HandleActorAddedToSequencer(AActor* Actor, const FGuid Binding);

	/** Callback for VR Editor mode exiting */
	void HandleVREditorModeExit();

private:

	/** Level sequence for our edit operation. */
	TObjectPtr<ULevelSequence> LevelSequence;

	/** Event that is cast when this toolkit is closed */
	FLevelSequenceEditorToolkitClosed OnClosedEvent;

	/** The sequencer used by this editor. */
	TSharedPtr<ISequencer> Sequencer;

	/** Pointer to the style set to use for toolkits. */
	TSharedRef<ISlateStyle> Style;

	/** Instance of a class used for managing the playback context for a level sequence. */
	TSharedPtr<FLevelSequencePlaybackContext> PlaybackContext;
private:

	/**	The tab ids for all the tabs used */
	static const FName SequencerMainTabId;
};
