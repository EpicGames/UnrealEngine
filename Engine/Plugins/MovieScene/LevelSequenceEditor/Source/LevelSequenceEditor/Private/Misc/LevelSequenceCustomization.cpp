// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LevelSequenceCustomization.h"

#include "ActorTreeItem.h"
#include "BakingAnimationKeySettings.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Engine/LevelStreaming.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "ISequencer.h"
#include "ISequencerModule.h"
#include "LevelSequenceDirector.h"
#include "LevelSequenceEditorCommands.h"
#include "LevelSequenceEditorSubsystem.h"
#include "LevelSequenceFBXInterop.h"
#include "Modules/ModuleManager.h"
#include "MovieScene.h"
#include "MovieSceneBindingReferences.h"
#include "MVVM/Extensions/IObjectBindingExtension.h"
#include "MVVM/Selection/Selection.h"
#include "MVVM/ViewModels/ObjectBindingModel.h"
#include "MVVM/ViewModels/SequencerEditorViewModel.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "ScopedTransaction.h"
#include "Selection.h"
#include "SequencerCommands.h"
#include "SequencerSettings.h"
#include "SequencerUtilities.h"
#include "Widgets/SLevelSequenceBakeTransform.h"

#define LOCTEXT_NAMESPACE "LevelSequenceCustomization"

namespace UE::Sequencer
{

void FLevelSequenceCustomization::AddCustomization(TUniquePtr<ISequencerCustomization> NewCustomization)
{
	AdditionalCustomizations.Add(MoveTemp(NewCustomization));
}

void FLevelSequenceCustomization::RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder)
{
	WeakSequencer = Builder.GetSequencer().AsShared();

	BindCommands();

	FSequencerCustomizationInfo Customization;
	Customization.ActionsMenuExtender = CreateActionsMenuExtender();
	Customization.OnBuildObjectBindingContextMenu = FOnGetSequencerMenuExtender::CreateRaw(this, &FLevelSequenceCustomization::CreateObjectBindingContextMenuExtender);
	Customization.OnBuildSidebarMenu = FOnGetSequencerMenuExtender::CreateRaw(this, &FLevelSequenceCustomization::CreateObjectBindingSidebarMenuExtender);
	Builder.AddCustomization(Customization);

	for(TUniquePtr<ISequencerCustomization>& ExternalCustomization : AdditionalCustomizations)
	{
		ExternalCustomization->RegisterSequencerCustomization(Builder);
	}	
}

void FLevelSequenceCustomization::UnregisterSequencerCustomization()
{
	for(TUniquePtr<ISequencerCustomization>& ExternalCustomization : AdditionalCustomizations)
	{
		ExternalCustomization->UnregisterSequencerCustomization();
	}

	CommandList = nullptr;
	WeakSequencer = nullptr;
}

void FLevelSequenceCustomization::BindCommands()
{
	const FLevelSequenceEditorCommands& Commands = FLevelSequenceEditorCommands::Get();

	CommandList = MakeShared<FUICommandList>().ToSharedPtr();

	CommandList->MapAction(
		Commands.ImportFBX,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::ImportFBX),
		FCanExecuteAction::CreateLambda([] { return true; }));

	CommandList->MapAction(
		Commands.ExportFBX,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::ExportFBX),
		FCanExecuteAction::CreateLambda([] { return true; }));

	constexpr int32 MinSectionsRequiredForSnapping = 1;
	CommandList->MapAction(
		Commands.SnapSectionsToTimelineUsingSourceTimecode,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::SnapSectionsToTimelineUsingSourceTimecode),
		FCanExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::DoesMovieSceneHaveEnoughSectionsSelected, MinSectionsRequiredForSnapping));

	constexpr int32 MinSectionsRequiredForSyncing = 2;
	CommandList->MapAction(
		Commands.SyncSectionsUsingSourceTimecode,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::SyncSectionsUsingSourceTimecode),
		FCanExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::DoesMovieSceneHaveEnoughSectionsSelected, MinSectionsRequiredForSyncing));

	CommandList->MapAction(
		Commands.FixActorReferences,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::FixActorReferences));

	CommandList->MapAction(
		Commands.BakeTransform,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::BakeTransform));

	CommandList->MapAction(
		Commands.AddActorsToBinding,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::AddActorsToBinding),
		FCanExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::AreAnyActorsSelected));

	CommandList->MapAction(
		Commands.ReplaceBindingWithActors,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::ReplaceBindingWithActors),
		FCanExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::AreAnyActorsSelected));

	CommandList->MapAction(
		Commands.RemoveActorsFromBinding,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::RemoveActorsFromBinding),
		FCanExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::AreAnyActorsSelected));

	CommandList->MapAction(
		Commands.RemoveAllBindings,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::RemoveAllBindings));

	CommandList->MapAction(
		Commands.RemoveInvalidBindings,
		FExecuteAction::CreateRaw(this, &FLevelSequenceCustomization::RemoveInvalidBindings));
}

TSharedPtr<FExtender> FLevelSequenceCustomization::CreateActionsMenuExtender()
{
	TSharedPtr<FExtender> ActionsMenuExtender = MakeShared<FExtender>();

	ActionsMenuExtender->AddMenuExtension(
		"SequenceOptions", EExtensionHook::First, CommandList,
		FMenuExtensionDelegate::CreateLambda([](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().BakeTransform);
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().ImportFBX);
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().ExportFBX);
			}));

	ActionsMenuExtender->AddMenuExtension(
		"Transform", EExtensionHook::After, CommandList,
		FMenuExtensionDelegate::CreateLambda([](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().SnapSectionsToTimelineUsingSourceTimecode);
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().SyncSectionsUsingSourceTimecode);
			}));

	ActionsMenuExtender->AddMenuExtension(
		"Bindings", EExtensionHook::First, CommandList,
		FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
			{
				MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().FixActorReferences);
			}));

	return ActionsMenuExtender;
}

void FLevelSequenceCustomization::ImportFBX()
{
	FLevelSequenceFBXInterop Interop(WeakSequencer.Pin().ToSharedRef());
	Interop.ImportFBX();
}

void FLevelSequenceCustomization::ExportFBX()
{
	FLevelSequenceFBXInterop Interop(WeakSequencer.Pin().ToSharedRef());
	Interop.ExportFBX();
}

void FLevelSequenceCustomization::SnapSectionsToTimelineUsingSourceTimecode()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		TArray<UMovieSceneSection*> Sections;
		Sequencer->GetSelectedSections(Sections);
		FSequencerUtilities::SnapSectionsToTimelineUsingSourceTimecode(Sequencer.ToSharedRef(), Sections);
	}
}

void FLevelSequenceCustomization::SyncSectionsUsingSourceTimecode()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		TArray<UMovieSceneSection*> Sections;
		Sequencer->GetSelectedSections(Sections);
		FSequencerUtilities::SyncSectionsUsingSourceTimecode(Sequencer.ToSharedRef(), Sections);
	}
}

bool FLevelSequenceCustomization::DoesMovieSceneHaveEnoughSectionsSelected(int32 MinSectionCount)
{
	const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return false;
	}

	TArray<UMovieSceneSection*> SelectedSections;
	Sequencer->GetSelectedSections(SelectedSections);

	return (SelectedSections.Num() >= MinSectionCount);
}

void FLevelSequenceCustomization::FixActorReferences()
{
	if (const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		FSequencerUtilities::FixActorReferences(Sequencer.ToSharedRef());
	}
}

void FLevelSequenceCustomization::BakeTransform()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence ? Sequence->GetMovieScene() : nullptr;
	if (!MovieScene)
	{
		return;
	}

	TArray<FGuid> ObjectBindings;
	Sequencer->GetSelectedObjects(ObjectBindings);

	TArray<FMovieSceneBindingProxy> BindingProxies;
	for (FGuid Guid : ObjectBindings)
	{
		BindingProxies.Add(FMovieSceneBindingProxy(Guid, Sequence));
	}

	static FBakingAnimationKeySettings Settings; //reuse the settings except for the range
	Settings.StartFrame = UE::MovieScene::DiscreteInclusiveLower(MovieScene->GetPlaybackRange());
	Settings.EndFrame = UE::MovieScene::DiscreteExclusiveUpper(MovieScene->GetPlaybackRange());

	TSharedRef<SLevelSequenceBakeTransform> BakeWidget =
		SNew(SLevelSequenceBakeTransform)
		.Settings(Settings)
		.Sequencer(Sequencer.Get())
		.OnBake_Lambda([this, &BindingProxies, Sequencer](FBakingAnimationKeySettings InSettings)
			{
				if (FSequencerUtilities::BakeTransform(Sequencer.ToSharedRef(), BindingProxies, InSettings))
				{
					Settings = InSettings;
				}

				return FReply::Handled();
			});

	BakeWidget->OpenDialog(true);
}

TSharedPtr<FExtender> FLevelSequenceCustomization::CreateObjectBindingContextMenuExtender(FViewModelPtr InViewModel)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	TSharedPtr<FObjectBindingModel> ObjectBindingModel = InViewModel->CastThisShared<FObjectBindingModel>();
	Extender->AddMenuExtension(
			"ObjectBindingActions", EExtensionHook::Before, CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FLevelSequenceCustomization::ExtendObjectBindingContextMenu, ObjectBindingModel));
	return Extender.ToSharedPtr();
}

void FLevelSequenceCustomization::ExtendObjectBindingContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FObjectBindingModel> ObjectBindingModel)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	TSharedPtr<FSequencerEditorViewModel> EditorViewModel = Sequencer->GetViewModel();

	FGuid ObjectBindingID = ObjectBindingModel->GetObjectGuid();
	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* MovieScene = Sequence->GetMovieScene();

	if (!MovieScene || !ObjectBindingID.IsValid())
	{
		return;
	}
	bool bShowConvert = true;

	if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID))
	{
		// We can't convert sub-objects to different binding types for now.
		if (Possessable->GetParent().IsValid())
		{
			bShowConvert = false;
		}
		bool bCustomBinding = false;
		bool bMultipleBindings = false;
		UObject* ResolutionContext = MovieSceneHelpers::GetResolutionContext(Sequence, ObjectBindingID, Sequencer->GetFocusedTemplateID(), Sequencer->GetSharedPlaybackState());

		if (const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences())
		{
			bCustomBinding = Algo::AnyOf(BindingReferences->GetReferences(ObjectBindingID), [](const FMovieSceneBindingReference& Reference) { return Reference.CustomBinding; });
			bMultipleBindings = BindingReferences->GetReferences(ObjectBindingID).Num() > 1;
			UE::UniversalObjectLocator::FResolveParams LocatorResolveParams(ResolutionContext);
			FMovieSceneBindingResolveParams BindingResolveParams{ Sequence, ObjectBindingID, Sequencer->GetFocusedTemplateID(), ResolutionContext };

			// Can convert to possessable
			int32 BindingIndex = 0;
			bool bAnyValidConversions = false;
			if (Algo::AnyOf(BindingReferences->GetReferences(ObjectBindingID), [&BindingIndex, Sequencer](const FMovieSceneBindingReference& BindingReference) {
				return FSequencerUtilities::CanConvertToPossessable(Sequencer.ToSharedRef(), BindingReference.ID, BindingIndex++);
				}))
			{
				bAnyValidConversions = true;
			}
			else
			{
				TArrayView<const TSubclassOf<UMovieSceneCustomBinding>> PrioritySortedCustomBindingTypes = Sequencer->GetSupportedCustomBindingTypes();
				for (const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType : PrioritySortedCustomBindingTypes)
				{
					BindingIndex = 0;
					if (Algo::AllOf(BindingReferences->GetReferences(ObjectBindingID), [&BindingIndex, &CustomBindingType, Sequencer](const FMovieSceneBindingReference& BindingReference)
						{
							return FSequencerUtilities::CanConvertToCustomBinding(Sequencer.ToSharedRef(), BindingReference.ID, CustomBindingType, BindingIndex++);
						}))
					{
						bAnyValidConversions = true;
						break;
					}
				}
			}
			if (!bAnyValidConversions)
			{
				bShowConvert = false;
			}
		}

		MenuBuilder.AddSubMenu(
			LOCTEXT("BindingProperties", "Binding Properties"),
			LOCTEXT("BindingPropertiesTooltip", "Modify the actor and object bindings for this track"),
			FNewMenuDelegate::CreateRaw(this, &FLevelSequenceCustomization::AddBindingPropertiesSubMenu));

		// Regular possessable
		if (!bCustomBinding)
		{
			MenuBuilder.BeginSection("Possessable");

			if (IsSelectedBindingRootPossessable())
			{
				MenuBuilder.AddSubMenu(
					LOCTEXT("AssignActor", "Assign Actor"),
					LOCTEXT("AssignActorTooltip", "Assign an actor to this track"),
					FNewMenuDelegate::CreateRaw(this, &FLevelSequenceCustomization::AddAssignActorSubMenu));
			}

			MenuBuilder.EndSection();
		}
		else
		{
			MenuBuilder.BeginSection("CustomBinding");
			bool bCustomSpawnable = MovieSceneHelpers::SupportsObjectTemplate(Sequence, ObjectBindingID, Sequencer->GetSharedPlaybackState());
			// Check for custom binding types

			if (bCustomSpawnable)
			{
				MenuBuilder.AddMenuEntry(FSequencerCommands::Get().SaveCurrentSpawnableState);

				if (!bMultipleBindings)
				{
					MenuBuilder.AddSubMenu(
						LOCTEXT("ChangeClassLabel", "Change Class"),
						LOCTEXT("ChangeClassTooltip", "Change the class (object template) that this spawns from"),
						FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
						{
							const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
							if (!Sequencer.IsValid())
							{
								return;
							}

							UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();

							TArray<FSequencerChangeBindingInfo> Bindings;
							const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();
							for (TViewModelPtr<IObjectBindingExtension> ObjectBindingNode : Sequencer->GetViewModel()->GetSelection()->Outliner.Filter<IObjectBindingExtension>())
							{
								int32 BindingIndex = 0;
								for (const FMovieSceneBindingReference& Reference : BindingReferences->GetReferences(ObjectBindingNode->GetObjectGuid()))
								{
									Bindings.Add({ Reference.ID, BindingIndex++ });
								}
							}

							FSequencerUtilities::AddChangeClassMenu(MenuBuilder, Sequencer.ToSharedRef(), Bindings, TFunction<void()>());
						}));
				}
			}

			MenuBuilder.EndSection();
		}
	}
	
	if (bShowConvert)
	{
		MenuBuilder.BeginSection("ConvertBinding");

		MenuBuilder.AddSubMenu(
			LOCTEXT("ConvertBindingLabel", "Convert Selected Binding(s) To..."),
			LOCTEXT("ConvertBindingLabelTooltip", "Convert selected bindings into another binding type"),
			FNewMenuDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
				{
					const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
					if (!Sequencer)
					{
						return;
					}

					UMovieSceneSequence* const Sequence = Sequencer->GetFocusedMovieSceneSequence();
					if (!Sequence)
					{
						return;
					}

					const FMovieSceneBindingReferences* BindingReferences = Sequence->GetBindingReferences();
					if (!BindingReferences)
					{
						return;
					}

					TArray<FGuid> ObjectBindings;
					Sequencer->GetSelectedObjects(ObjectBindings);
					if (ObjectBindings.IsEmpty())
					{
						return;
					}

					TArray<FSequencerChangeBindingInfo> Bindings;
					for (FGuid ObjectGuid : ObjectBindings)
					{
						int32 BindingIndex = 0;
						for (const FMovieSceneBindingReference& Reference : BindingReferences->GetReferences(ObjectGuid))
						{
							Bindings.Add({ Reference.ID, BindingIndex++ });
						}
					}

					GEditor->GetEditorSubsystem<ULevelSequenceEditorSubsystem>()->AddChangeBindingTypeMenu(MenuBuilder, Sequencer.ToSharedRef(), Bindings, true, TFunction<void()>());
				}));

		MenuBuilder.EndSection();
	}

	MenuBuilder.BeginSection("Import/Export", LOCTEXT("ImportExportMenuSectionName", "Import/Export"));

	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().BakeTransform);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ImportFBX", "Import..."),
		LOCTEXT("ImportFBXTooltip", "Import FBX animation to this object"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] {
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}
				
				FLevelSequenceFBXInterop Interop(Sequencer.ToSharedRef());
					Interop.ImportFBXOntoSelectedNodes();
				})
		));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ExportFBX", "Export..."),
		LOCTEXT("ExportFBXTooltip", "Export FBX animation from this object"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this] {
				const TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
				if (!Sequencer.IsValid())
				{
					return;
				}
				
				FLevelSequenceFBXInterop Interop(Sequencer.ToSharedRef());
					Interop.ExportFBX();
				})
		));

	MenuBuilder.EndSection();
}

void FLevelSequenceCustomization::AddAssignActorSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().AddActorsToBinding);
	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().ReplaceBindingWithActors);
	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().RemoveActorsFromBinding);
	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().RemoveAllBindings);
	MenuBuilder.AddMenuEntry(FLevelSequenceEditorCommands::Get().RemoveInvalidBindings);

	// Set up a menu entry to assign an actor to the object binding node
	TSharedRef<SWidget> ActorPickerWidget = BuildActorPicker();

	MenuBuilder.AddMenuSeparator();
	MenuBuilder.AddWidget(ActorPickerWidget, FText::GetEmpty(), true);
}

void FLevelSequenceCustomization::AddBindingPropertiesSubMenu(FMenuBuilder& MenuBuilder)
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		if (ULevelSequenceEditorSubsystem* SubSystem = GEditor->GetEditorSubsystem<ULevelSequenceEditorSubsystem>())
		{
			TSharedRef<SWidget> DetailsView = SubSystem->BuildBindingPropertiesDetailsWidget(Sequencer.ToSharedRef());
			MenuBuilder.AddMenuSeparator();
			MenuBuilder.AddWidget(DetailsView, FText::GetEmpty(), true);
		}
	}
}

TSharedRef<SWidget> FLevelSequenceCustomization::BuildActorPicker()
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return SNullWidget::NullWidget;
	}

	TArray<FGuid> ObjectBindings;
	Sequencer->GetSelectedObjects(ObjectBindings);
	if (ObjectBindings.IsEmpty())
	{
		return SNullWidget::NullWidget;
	}
	const FGuid SelectedObjectBinding = ObjectBindings[0];

	// Find the set of already bound actors and create a filter to exclude those from the actor picker widget
	TSet<const AActor*> AlreadyBoundActors;
	for (TWeakObjectPtr<> Ptr : Sequencer->FindObjectsInCurrentSequence(SelectedObjectBinding))
	{
		if (const AActor* Actor = Cast<AActor>(Ptr.Get()))
		{
			AlreadyBoundActors.Add(Actor);
		}
	}

	auto IsActorValidForAssignment = [AlreadyBoundActors](const AActor* InActor)
	{
		return !AlreadyBoundActors.Contains(InActor);
	};

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = false;
	InitOptions.bShowSearchBox = true;
	InitOptions.bShowCreateNewFolder = false;
	InitOptions.bFocusSearchBoxWhenOpened = true;

	// Only display the actor label column
	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

	// Only display actors that are not possessed already
	InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(IsActorValidForAssignment));

	const float WidthOverride = Sequencer->GetSequencerSettings() ? Sequencer->GetSequencerSettings()->GetAssetBrowserWidth() : 500.0f;
	const float HeightOverride = Sequencer->GetSequencerSettings() ? Sequencer->GetSequencerSettings()->GetAssetBrowserHeight() : 400.0f;

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");

	return SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([Sequencer, SelectedObjectBinding](AActor* Actor)
				{
					// Create a new binding for this actor
					FSlateApplication::Get().DismissAllMenus();
					FSequencerUtilities::AssignActor(Sequencer.ToSharedRef(), Actor, SelectedObjectBinding);
				})
			)
		];
}

void FLevelSequenceCustomization::AddActorsToBinding()
{
	TArray<AActor*> SelectedActors = GetSelectedActors();
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();

	if (WeakSequencer.IsValid() && BindingProxy.BindingID.IsValid())
	{
		FSequencerUtilities::AddActorsToBinding(WeakSequencer.Pin().ToSharedRef(), SelectedActors, BindingProxy);
	}
}

void FLevelSequenceCustomization::ReplaceBindingWithActors()
{
	TArray<AActor*> SelectedActors = GetSelectedActors();
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();

	if (WeakSequencer.IsValid() && BindingProxy.BindingID.IsValid())
	{
		FSequencerUtilities::ReplaceBindingWithActors(WeakSequencer.Pin().ToSharedRef(), SelectedActors, BindingProxy);
	}
}

void FLevelSequenceCustomization::RemoveActorsFromBinding()
{
	TArray<AActor*> SelectedActors = GetSelectedActors();
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();

	if (WeakSequencer.IsValid() && BindingProxy.BindingID.IsValid())
	{
		FSequencerUtilities::RemoveActorsFromBinding(WeakSequencer.Pin().ToSharedRef(), SelectedActors, BindingProxy);
	}
}

void FLevelSequenceCustomization::RemoveAllBindings()
{
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();
	if (WeakSequencer.IsValid() && BindingProxy.BindingID.IsValid())
	{
		FSequencerUtilities::RemoveAllBindings(WeakSequencer.Pin().ToSharedRef(), BindingProxy);
	}
}

void FLevelSequenceCustomization::RemoveInvalidBindings()
{
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();
	if (WeakSequencer.IsValid() && BindingProxy.BindingID.IsValid())
	{
		FSequencerUtilities::RemoveInvalidBindings(WeakSequencer.Pin().ToSharedRef(), BindingProxy);
	}
}

FMovieSceneBindingProxy FLevelSequenceCustomization::GetSelectedBindingProxy()
{
	if (TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin())
	{
		TArray<FGuid> ObjectBindings;
		Sequencer->GetSelectedObjects(ObjectBindings);
		if (!ObjectBindings.IsEmpty())
		{
			return FMovieSceneBindingProxy(ObjectBindings[0], Sequencer->GetFocusedMovieSceneSequence());
		}
	}

	return FMovieSceneBindingProxy();
}

TArray<AActor*> FLevelSequenceCustomization::GetSelectedActors()
{
	TArray<AActor*> SelectedActors;
	if (USelection* ActorSelection = GEditor->GetSelectedActors())
	{
		ActorSelection->GetSelectedObjects<AActor>(SelectedActors);
	}
	return SelectedActors;
}

bool FLevelSequenceCustomization::AreAnyActorsSelected()
{
	TArray<AActor*> SelectedActors = GetSelectedActors();
	return !SelectedActors.IsEmpty();
}

bool FLevelSequenceCustomization::IsSelectedBindingRootPossessable()
{
	const FMovieSceneBindingProxy BindingProxy = GetSelectedBindingProxy();
	UMovieScene* MovieScene = BindingProxy.GetMovieScene();

	if (BindingProxy.BindingID.IsValid() && MovieScene)
	{
		if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(BindingProxy.BindingID))
		{
			if (!Possessable->GetParent().IsValid() && !Possessable->GetSpawnableObjectBindingID().IsValid())
			{
				return true;
			}
		}
	}
	return false;
}

TSharedPtr<FExtender> FLevelSequenceCustomization::CreateObjectBindingSidebarMenuExtender(FViewModelPtr InViewModel)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();
	
	TSharedPtr<FObjectBindingModel> ObjectBindingModel = InViewModel->CastThisShared<FObjectBindingModel>();
	
	Extender->AddMenuExtension(TEXT("ObjectBindingActions"), EExtensionHook::Before, nullptr,
		FMenuExtensionDelegate::CreateRaw(this, &FLevelSequenceCustomization::ExtendObjectBindingSidebarMenu, ObjectBindingModel));
	
	return Extender.ToSharedPtr();
}

void FLevelSequenceCustomization::ExtendObjectBindingSidebarMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FObjectBindingModel> ObjectBindingModel)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer)
	{
		return;
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	if (!Sequence)
	{
		return;
	}

	UMovieScene* MovieScene = Sequence->GetMovieScene();

	FGuid ObjectBindingID = ObjectBindingModel->GetObjectGuid();

	if (!MovieScene || !ObjectBindingID.IsValid())
	{
		return;
	}

	if (FMovieScenePossessable* Possessable = MovieScene->FindPossessable(ObjectBindingID))
	{
		AddBindingPropertiesSubMenu(MenuBuilder);
	}
}

} // namespace UE::Sequencer

#undef LOCTEXT_NAMESPACE
