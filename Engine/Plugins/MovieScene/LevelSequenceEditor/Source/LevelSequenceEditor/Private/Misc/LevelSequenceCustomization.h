// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MVVM/ViewModelPtr.h"
#include "SequencerCustomizationManager.h"
#include "Toolkits/AssetEditorToolkit.h"

namespace UE::Sequencer
{
	class FObjectBindingModel;
	class FSequencerEditorViewModel;

/**
 * The sequencer customization for level sequences. 
 */
class FLevelSequenceCustomization : public ISequencerCustomization
{
public:
	void AddCustomization(TUniquePtr<ISequencerCustomization> NewCustomization);
	
protected:

	//~ ISequencerCustomization interface
	virtual void RegisterSequencerCustomization(FSequencerCustomizationBuilder& Builder) override;
	virtual void UnregisterSequencerCustomization() override;

private:

	/** Map the UI commands used by the extensions in this customization */
	void BindCommands();

	/** Create the menu extenders that will be added to the registered customization builder */
	TSharedPtr<FExtender> CreateActionsMenuExtender();
	TSharedPtr<FExtender> CreateObjectBindingContextMenuExtender(FViewModelPtr InViewModel);
	TSharedPtr<FExtender> CreateObjectBindingSidebarMenuExtender(FViewModelPtr InViewModel);

	/** Action Menu Extensions */
	void ImportFBX();
	void ExportFBX();
	void SnapSectionsToTimelineUsingSourceTimecode();
	void SyncSectionsUsingSourceTimecode();
	void FixActorReferences();
	void BakeTransform();
	
	bool DoesMovieSceneHaveEnoughSectionsSelected(int32 MinSectionCount);

	/** Object Binding Context Menu Extensions */
	void ExtendObjectBindingContextMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FObjectBindingModel> ObjectBindingModel);
	void AddAssignActorSubMenu(FMenuBuilder& MenuBuilder);
	void AddBindingPropertiesSubMenu(FMenuBuilder& MenuBuilder);
	TSharedRef<SWidget> BuildActorPicker();

	void AddActorsToBinding();
	void ReplaceBindingWithActors();
	void RemoveActorsFromBinding();
	void RemoveAllBindings();
	void RemoveInvalidBindings();

	FMovieSceneBindingProxy GetSelectedBindingProxy();
	TArray<AActor*> GetSelectedActors();

	bool AreAnyActorsSelected();
	bool IsSelectedBindingRootPossessable();

	/** Object Binding Sidebar Menu Extensions */
	void ExtendObjectBindingSidebarMenu(FMenuBuilder& MenuBuilder, TSharedPtr<FObjectBindingModel> ObjectBindingModel);

private:

	TWeakPtr<ISequencer> WeakSequencer;

	TSharedPtr<FUICommandList> CommandList;

	TArray<TUniquePtr<ISequencerCustomization>>	AdditionalCustomizations;
};

} // namespace UE::Sequencer

