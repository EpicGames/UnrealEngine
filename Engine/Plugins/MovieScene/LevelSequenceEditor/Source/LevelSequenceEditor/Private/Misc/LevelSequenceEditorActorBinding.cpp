// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorActorBinding.h"
#include "ISceneOutliner.h"
#include "ISequencer.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieScenePossessable.h"
#include "Styling/SlateIconFinder.h"
#include "SceneOutlinerModule.h"
#include "ActorTreeItem.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Misc/EditorPathHelper.h"
#include "Editor.h"
#include "Selection.h"
#include "SequencerSettings.h"

#define LOCTEXT_NAMESPACE "LevelSequenceEditorActorBinding"

FLevelSequenceEditorActorBinding::FLevelSequenceEditorActorBinding(TSharedRef<ISequencer> InSequencer)
	: Sequencer(InSequencer)
{
}

FText FLevelSequenceEditorActorBinding::GetDisplayName() const
{
	return LOCTEXT("ActorTrackEditor_DisplayName", "Actor");
}

void FLevelSequenceEditorActorBinding::BuildSequencerAddMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddSubMenu(
		LOCTEXT("AddActor_Label", "Add Actor Track"),
		LOCTEXT("AddActor_ToolTip", "Allow sequencer to possess an actor that already exists in the current level"),
		FNewMenuDelegate::CreateRaw(this, &FLevelSequenceEditorActorBinding::AddPossessActorMenuExtensions),
		false /*bInOpenSubMenuOnClick*/,
		FSlateIcon("LevelSequenceEditorStyle", "LevelSequenceEditor.PossessNewActor")
		);
}

bool FLevelSequenceEditorActorBinding::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	return InSequence->GetClass()->IsChildOf(ULevelSequence::StaticClass());
}

void FLevelSequenceEditorActorBinding::AddPossessActorMenuExtensions(FMenuBuilder& MenuBuilder)
{
	// This is called for every actor in the map, and asking the sequencer for a handle to the object to check if we have
	// already bound is an issue on maps that have tens of thousands of actors. The current sequence is will almost always
	// have actors than the map, so instead we'll cache off all of the actors already bound and check against that map locally.
	// This list is checked via an async filter, but we don't need to store them as weak pointers because we're doing a direct
	// pointer comparison and not an object comparison, and the async list shouldn't run the filter if the object is no longer valid.
	// We don't need to check against Sequencer spawnables as they're not valid for possession.
	TSet<UObject*> ExistingPossessedObjects;
	UMovieSceneSequence* MovieSceneSequence = nullptr;
	TSharedPtr<ISequencer> SequencerPtr = Sequencer.Pin();
	if (SequencerPtr.IsValid())
	{
		MovieSceneSequence = SequencerPtr->GetFocusedMovieSceneSequence();
		UMovieScene* MovieScene = MovieSceneSequence->GetMovieScene();
		if(MovieScene)
		{
			for (int32 Index = 0; Index < MovieScene->GetPossessableCount(); Index++)
			{
				// Only get bound objects for top-level possessables.
				FMovieScenePossessable& Possessable = MovieScene->GetPossessable(Index);
				if (!Possessable.GetParent().IsValid())
				{
					// A possession guid can apply to more than one object, so we get all bound objects for the GUID and add them to our set.
					TArray<UObject*, TInlineAllocator<1>> OutObjects;
					MovieSceneSequence->LocateBoundObjects(Possessable.GetGuid(), UE::UniversalObjectLocator::FResolveParams(SequencerPtr->GetPlaybackContext()), SequencerPtr->FindSharedPlaybackState(), OutObjects);
					ExistingPossessedObjects.Append(OutObjects);
				}
			}
		}
	}

	auto IsActorValidForPossession = [=](const AActor* InActor, TSet<UObject*> InPossessedObjectSet)
	{
		return !InPossessedObjectSet.Contains((UObject*)InActor);
	};

	// Set up a menu entry to add the selected actor(s) to the sequencer
	TArray<AActor*> ActorsValidForPossession;
	GEditor->GetSelectedActors()->GetSelectedObjects(ActorsValidForPossession);
	ActorsValidForPossession.RemoveAll([&](AActor* In){ return !IsActorValidForPossession(In, ExistingPossessedObjects); });

	FText SelectedLabel;
	FSlateIcon ActorIcon = FSlateIconFinder::FindIconForClass(AActor::StaticClass());
	if (ActorsValidForPossession.Num() == 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddSpecificActor", "Add '{0}'"), FText::FromString(ActorsValidForPossession[0]->GetActorLabel()));
		ActorIcon = FSlateIconFinder::FindIconForClass(ActorsValidForPossession[0]->GetClass());
	}
	else if (ActorsValidForPossession.Num() > 1)
	{
		SelectedLabel = FText::Format(LOCTEXT("AddCurrentActorSelection", "Add Current Selection ({0} actors)"), FText::AsNumber(ActorsValidForPossession.Num()));
	}

	if (!SelectedLabel.IsEmpty())
	{
		// Copy the array into the lambda - probably not that big a deal
		MenuBuilder.AddMenuEntry(SelectedLabel, FText(), ActorIcon, FExecuteAction::CreateLambda([this, ActorsValidForPossession]{
			FSlateApplication::Get().DismissAllMenus();
			AddActorsToSequencer(ActorsValidForPossession.GetData(), ActorsValidForPossession.Num());
		}));
	}

	// Add an entry for an empty binding
	MenuBuilder.AddMenuEntry(LOCTEXT("EmptyBinding", "New Empty Binding"),
		LOCTEXT("EmptyBindingTooltip", "Add a new empty binding to Sequencer which can be connected to an object or actor afterwards in the Binding Properties"),
		FSlateIcon(), // TODO: empty icon?
		FExecuteAction::CreateLambda([this] {
			FSlateApplication::Get().DismissAllMenus();
			Sequencer.Pin()->AddEmptyBinding();
			}));

	MenuBuilder.BeginSection("ChooseActorSection", LOCTEXT("ChooseActor", "Choose Actor:"));

	// Set up a menu entry to add any arbitrary actor to the sequencer
	FSceneOutlinerInitializationOptions InitOptions;
	{
		// We hide the header row to keep the UI compact.
		InitOptions.bShowHeaderRow = false;
		InitOptions.bShowSearchBox = true;
		InitOptions.bShowCreateNewFolder = false;
		InitOptions.bFocusSearchBoxWhenOpened = true;

		// Allow transient actors if the level sequence itself is transient (the expectation is that these would never be saved)
		InitOptions.bShowTransient = MovieSceneSequence && MovieSceneSequence->GetOutermost() == GetTransientPackage();

		// Only want the actor label column
		InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 0));

		// Only display actors that are not possessed already
		InitOptions.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateLambda(IsActorValidForPossession, ExistingPossessedObjects));
	}

	const bool bHideLevelInstanceHierarchy = !FEditorPathHelper::IsEnabled();

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	// actor selector to allow the user to choose an actor
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	TSharedRef< SWidget > MiniSceneOutliner =
		SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			SceneOutlinerModule.CreateActorPicker(
				InitOptions,
				FOnActorPicked::CreateLambda([this](AActor* Actor){
					// Create a new binding for this actor
					FSlateApplication::Get().DismissAllMenus();
					AddActorsToSequencer(&Actor, 1);
				}),
				nullptr,
				bHideLevelInstanceHierarchy
			)
		];

	MenuBuilder.AddWidget(MiniSceneOutliner, FText::GetEmpty(), true);
	MenuBuilder.EndSection();
}

void FLevelSequenceEditorActorBinding::AddActorsToSequencer(AActor*const* InActors, int32 NumActors)
{
	TArray<TWeakObjectPtr<AActor>> Actors;

	while (NumActors--)
	{
		AActor* ThisActor = *InActors;
		Actors.Add(ThisActor);

		InActors++;
	}

	Sequencer.Pin()->AddActors(Actors);
}

#undef LOCTEXT_NAMESPACE
