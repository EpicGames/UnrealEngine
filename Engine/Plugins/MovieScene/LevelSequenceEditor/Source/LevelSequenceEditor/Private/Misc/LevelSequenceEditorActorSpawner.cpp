// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelSequenceEditorActorSpawner.h"
#include "Engine/Blueprint.h"
#include "ActorFactories/ActorFactory.h"
#include "AssetSelection.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Level.h"
#include "LevelEditorViewport.h"
#include "SnappingUtils.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "SequencerSettings.h"
#include "IMovieScenePlaybackClient.h"
#include "ISequencer.h"
#include "MovieScene.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "TransformData.h"

#define LOCTEXT_NAMESPACE "LevelSequenceEditorActorSpawner"

TSharedRef<IMovieSceneObjectSpawner> FLevelSequenceEditorActorSpawner::CreateObjectSpawner()
{
	return MakeShareable(new FLevelSequenceEditorActorSpawner);
}

#if WITH_EDITOR

TValueOrError<FNewSpawnable, FText> FLevelSequenceEditorActorSpawner::CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory)
{
	FNewSpawnable NewSpawnable(nullptr, FName::NameToDisplayString(SourceObject.GetName(), false));

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject.GetFName());

	FText ErrorText;

	// Deal with creating a spawnable from an instance of an actor
	if (AActor* Actor = Cast<AActor>(&SourceObject))
	{
		// If the source actor is not transactional, temporarily add the flag to ensure that the duplicated object is created with the transactional flag.
		// This is necessary for the creation of the object to exist in the transaction buffer for multi-user workflows
		const bool bWasTransactional = Actor->HasAnyFlags(RF_Transactional);
		if (!bWasTransactional)
		{
			Actor->SetFlags(RF_Transactional);
		}

		AActor* SpawnedActor = Cast<AActor>(StaticDuplicateObject(Actor, &OwnerMovieScene, TemplateName, RF_AllFlags));
		SpawnedActor->DetachFromActor(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
		SpawnedActor->bIsEditorPreviewActor = false;
		NewSpawnable.ObjectTemplate = SpawnedActor;
		NewSpawnable.Name = Actor->GetActorLabel();

		if (!bWasTransactional)
		{
			Actor->ClearFlags(RF_Transactional);
		}
	}

	// If it's a blueprint, we need some special handling
	else if (UBlueprint* SourceBlueprint = Cast<UBlueprint>(&SourceObject))
	{
		if (!OwnerMovieScene.GetClass()->IsChildOf(SourceBlueprint->GeneratedClass->ClassWithin))
		{
			ErrorText = FText::Format(LOCTEXT("ClassWithin", "Unable to add spawnable for class of type '{0}' since it has a required outer class '{1}'."), FText::FromString(SourceObject.GetName()), FText::FromString(SourceBlueprint->GeneratedClass->ClassWithin->GetName()));
			return MakeError(ErrorText);
		}

		NewSpawnable.ObjectTemplate = NewObject<UObject>(&OwnerMovieScene, SourceBlueprint->GeneratedClass, TemplateName, RF_Transactional);
	}

	else if (UBlueprintGeneratedClass* SourceBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(&SourceObject))
	{
		if (UBlueprint* BlueprintGeneratedBy = Cast<UBlueprint>(SourceBlueprintGeneratedClass->ClassGeneratedBy))
		{
			if (!OwnerMovieScene.GetClass()->IsChildOf(BlueprintGeneratedBy->GeneratedClass->ClassWithin))
			{
				ErrorText = FText::Format(LOCTEXT("ClassWithin", "Unable to add spawnable for class of type '{0}' since it has a required outer class '{1}'."), FText::FromString(SourceObject.GetName()), FText::FromString(BlueprintGeneratedBy->GeneratedClass->ClassWithin->GetName()));
				return MakeError(ErrorText);
			}

			NewSpawnable.ObjectTemplate = NewObject<UObject>(&OwnerMovieScene, BlueprintGeneratedBy->GeneratedClass, TemplateName, RF_Transactional);
		}
	}

	// At this point we have to assume it's an asset
	else
	{
		UActorFactory* FactoryToUse = ActorFactory ? ActorFactory : FActorFactoryAssetProxy::GetFactoryForAssetObject(&SourceObject);
		if (!FactoryToUse)
		{
			ErrorText = FText::Format(LOCTEXT("CouldNotFindFactory", "Unable to create spawnable from asset '{0}' - no valid factory could be found."), FText::FromString(SourceObject.GetName()));
		}

		if (FactoryToUse)
		{
			if (!FactoryToUse->CanCreateActorFrom(FAssetData(&SourceObject), ErrorText))
			{
				if (!ErrorText.IsEmpty())
				{
					ErrorText = FText::Format(LOCTEXT("CannotCreateActorFromAsset_Ex", "Unable to create spawnable from  asset '{0}'. {1}."), FText::FromString(SourceObject.GetName()), ErrorText);
				}
				else
				{
					ErrorText = FText::Format(LOCTEXT("CannotCreateActorFromAsset", "Unable to create spawnable from  asset '{0}'."), FText::FromString(SourceObject.GetName()));
				}
			}

			UWorld* World = GCurrentLevelEditingViewportClient ? GCurrentLevelEditingViewportClient->GetWorld() : nullptr;
			if (World)
			{
				const FName ActorName = MakeUniqueObjectName(World->PersistentLevel, FactoryToUse->NewActorClass->StaticClass(), TemplateName);

				FActorSpawnParameters SpawnParams;
				SpawnParams.ObjectFlags = RF_Transient | RF_Transactional;
				SpawnParams.Name = ActorName;

				AActor* Instance = FactoryToUse->CreateActor(&SourceObject, World->PersistentLevel, FTransform(), SpawnParams);
				if (Instance)
				{
					Instance->bIsEditorPreviewActor = false;
					NewSpawnable.ObjectTemplate = StaticDuplicateObject(Instance, &OwnerMovieScene, TemplateName, RF_AllFlags & ~RF_Transient);

					const bool bNetForce = false;
					const bool bShouldModifyLevel = false;
					World->DestroyActor(Instance, bNetForce, bShouldModifyLevel);
				}
			}
		}
	}

	if (!NewSpawnable.ObjectTemplate || !NewSpawnable.ObjectTemplate->IsA<AActor>())
	{
		if (UClass* InClass = Cast<UClass>(&SourceObject))
		{
			if (!InClass->IsChildOf(AActor::StaticClass()))
			{
				ErrorText = FText::Format(LOCTEXT("NotAnActorClass", "Unable to add spawnable for class of type '{0}' since it is not a valid actor class."), FText::FromString(InClass->GetName()));
				return MakeError(ErrorText);
			}

			NewSpawnable.ObjectTemplate = NewObject<UObject>(&OwnerMovieScene, InClass, TemplateName, RF_Transactional);
		}

		if (!NewSpawnable.ObjectTemplate || !NewSpawnable.ObjectTemplate->IsA<AActor>())
		{
			if (ErrorText.IsEmpty())
			{
				ErrorText = FText::Format(LOCTEXT("UnknownClassError", "Unable to create a new spawnable object from {0}."), FText::FromString(SourceObject.GetName()));
			}

			return MakeError(ErrorText);
		}
	}

	return MakeValue(NewSpawnable);
}

static void PlaceActorInFrontOfCamera(AActor* ActorCDO)
{
	// Place the actor in front of the active perspective camera if we have one
	if ((GCurrentLevelEditingViewportClient != nullptr) && GCurrentLevelEditingViewportClient->IsPerspective())
	{
		// Don't allow this when the active viewport is showing a simulation/PIE level
		const bool bIsViewportShowingPIEWorld = GCurrentLevelEditingViewportClient->GetWorld()->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
		if (!bIsViewportShowingPIEWorld)
		{
			// @todo sequencer actors: Ideally we could use the actor's collision to figure out how far to push out
			// the object (like when placing in viewports), but we can't really do that because we're only dealing with a CDO
			const float DistanceFromCamera = 50.0f;

			// Find a place to put the object
			// @todo sequencer cleanup: This code should be reconciled with the GEditor->MoveActorInFrontOfCamera() stuff
			const FVector& CameraLocation = GCurrentLevelEditingViewportClient->GetViewLocation();
			FRotator CameraRotation = GCurrentLevelEditingViewportClient->GetViewRotation();
			const FVector CameraDirection = CameraRotation.Vector();

			FVector NewLocation = CameraLocation + CameraDirection * (DistanceFromCamera + GetDefault<ULevelEditorViewportSettings>()->BackgroundDropDistance);
			FSnappingUtils::SnapPointToGrid(NewLocation, FVector::ZeroVector);

			CameraRotation.Roll = 0.f;
			CameraRotation.Pitch = 0.f;

			ActorCDO->SetActorRelativeLocation(NewLocation);
			ActorCDO->SetActorRelativeRotation(CameraRotation);
		}
	}
}

bool FLevelSequenceEditorActorSpawner::CanSetupDefaultsForSpawnable(UObject* SpawnedObject) const
{
	if (SpawnedObject == nullptr)
	{
		return true;
	}

	return FLevelSequenceActorSpawner::CanSetupDefaultsForSpawnable(SpawnedObject);
}

void FLevelSequenceEditorActorSpawner::SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings)
{
	TOptional<FTransformData> DefaultTransform = TransformData;

	AActor* SpawnedActor = Cast<AActor>(SpawnedObject);
	if (SpawnedActor)
	{
		// Place the new spawnable in front of the camera (unless we were automatically created from a PIE actor)
		if (Settings->GetSpawnPosition() == SSP_PlaceInFrontOfCamera)
		{
			PlaceActorInFrontOfCamera(SpawnedActor);
		}
		DefaultTransform.Reset();
		DefaultTransform.Emplace();
		DefaultTransform->Translation = SpawnedActor->GetActorLocation();
		DefaultTransform->Rotation = SpawnedActor->GetActorRotation();
		DefaultTransform->Scale = FVector(1.0f, 1.0f, 1.0f);

		Sequencer->OnActorAddedToSequencer().Broadcast(SpawnedActor, Guid);

		const bool bNotifySelectionChanged = true;
		const bool bDeselectBSP = true;
		const bool bWarnAboutTooManyActors = false;
		const bool bSelectEvenIfHidden = false;

		GEditor->SelectNone(bNotifySelectionChanged, bDeselectBSP, bWarnAboutTooManyActors);
		GEditor->SelectActor(SpawnedActor, true, bNotifySelectionChanged, bSelectEvenIfHidden);
	}
	else if (DefaultTransform.IsSet())
	{
		DefaultTransform->Scale = FVector(1.0f, 1.0f, 1.0f);
	}

	UMovieSceneSequence* Sequence = Sequencer->GetFocusedMovieSceneSequence();
	UMovieScene* OwnerMovieScene = Sequence->GetMovieScene();

	// Ensure it has a spawn track if it's an old style spawnable. Otherwise, a custom spawnable will handle adding a binding lifetime track elsewhere
	if (OwnerMovieScene->FindSpawnable(Guid))
	{
		UMovieSceneSpawnTrack* SpawnTrack = Cast<UMovieSceneSpawnTrack>(OwnerMovieScene->FindTrack(UMovieSceneSpawnTrack::StaticClass(), Guid, NAME_None));
		if (!SpawnTrack)
		{
			SpawnTrack = Cast<UMovieSceneSpawnTrack>(OwnerMovieScene->AddTrack(UMovieSceneSpawnTrack::StaticClass(), Guid));
		}

		if (SpawnTrack)
		{
			UMovieSceneBoolSection* SpawnSection = Cast<UMovieSceneBoolSection>(SpawnTrack->CreateNewSection());
			SpawnSection->GetChannel().SetDefault(true);
			if (Sequencer->GetInfiniteKeyAreas())
			{
				SpawnSection->SetRange(TRange<FFrameNumber>::All());
			}
			SpawnTrack->AddSection(*SpawnSection);
			SpawnTrack->SetObjectId(Guid);
		}
	}

	// Ensure it will spawn in the right place
	if (DefaultTransform.IsSet())
	{
		UMovieScene3DTransformTrack* TransformTrack = Cast<UMovieScene3DTransformTrack>(OwnerMovieScene->FindTrack(UMovieScene3DTransformTrack::StaticClass(), Guid, "Transform"));
		if (!TransformTrack)
		{
			TransformTrack = Cast<UMovieScene3DTransformTrack>(OwnerMovieScene->AddTrack(UMovieScene3DTransformTrack::StaticClass(), Guid));
		}

		if (TransformTrack && TransformTrack->GetAllSections().Num() == 0)
		{
			UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(TransformTrack->CreateNewSection());
			TransformTrack->AddSection(*TransformSection);

			FTransform TransformOrigin;

			const IMovieScenePlaybackClient* Client = Sequencer->GetPlaybackClient();
			const UObject* InstanceData = Client ? Client->GetInstanceData() : nullptr;
			const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

			const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
			if (bHasInterface)
			{
				// Retrieve the current origin
				TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);
			}

			FTransform Transform;
			Transform.SetLocation(DefaultTransform->Translation);
			Transform.SetRotation(DefaultTransform->Rotation.Quaternion());
			Transform.SetScale3D(DefaultTransform->Scale);

			Transform = Transform * TransformOrigin.Inverse();

			FVector Location = Transform.GetLocation();
			FVector Rotation = Transform.GetRotation().Euler();
			FVector Scale    = Transform.GetScale3D();

			// Set the section to be infinite if necessary
			if (Sequencer->GetInfiniteKeyAreas())
			{
				TransformSection->SetRange(TRange<FFrameNumber>::All());
			}

			// Set the section's default values to this default transform
			TArrayView<FMovieSceneDoubleChannel*> DoubleChannels = TransformSection->GetChannelProxy().GetChannels<FMovieSceneDoubleChannel>();
			DoubleChannels[0]->SetDefault(Location.X);
			DoubleChannels[1]->SetDefault(Location.Y);
			DoubleChannels[2]->SetDefault(Location.Z);

			DoubleChannels[3]->SetDefault(Rotation.X);
			DoubleChannels[4]->SetDefault(Rotation.Y);
			DoubleChannels[5]->SetDefault(Rotation.Z);

			DoubleChannels[6]->SetDefault(Scale.X);
			DoubleChannels[7]->SetDefault(Scale.Y);
			DoubleChannels[8]->SetDefault(Scale.Z);
		}
	}
}

#endif	// #if WITH_EDITOR

#undef LOCTEXT_NAMESPACE
