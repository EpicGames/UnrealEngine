// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceActorSpawner.h"

struct FNewSpawnable;

class FLevelSequenceEditorActorSpawner : public FLevelSequenceActorSpawner
{
public:

	static TSharedRef<IMovieSceneObjectSpawner> CreateObjectSpawner();

	// IMovieSceneObjectSpawner interface
	virtual bool IsEditor() const override { return true; }
#if WITH_EDITOR
	virtual TValueOrError<FNewSpawnable, FText> CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory = nullptr) override;
	virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) override;
	virtual bool CanSetupDefaultsForSpawnable(UObject* SpawnedObject) const override;
#endif	
};
