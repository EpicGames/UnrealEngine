// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LevelSequenceEditorHelpers.h"
#include "DetailsViewArgs.h"
#include "UObject/UObjectIterator.h"
#include "LevelSequence.h"
#include "Factories/Factory.h"
#include "IAssetTools.h"
#include "AssetToolsModule.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "LevelSequenceEditorModule.h"

#include "Misc/LevelSequenceEditorSettings.h"
#include "Widgets/Layout/SScrollBox.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "IDetailsView.h"
#include "MovieSceneToolsProjectSettings.h"
#include "PropertyEditorModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SEditableTextBox.h" // IWYU pragma: keep
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor.h"
#include "SPrimaryButton.h"

/* LevelSequenceEditorHelpers
 *****************************************************************************/

#define LOCTEXT_NAMESPACE "LevelSequenceEditorHelpers"

TWeakPtr<SWindow> LevelSequenceWithShotsSettingsWindow;

class SLevelSequenceWithShotsSettings : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SLevelSequenceWithShotsSettings) {}
		SLATE_ARGUMENT(ULevelSequenceWithShotsSettings*, LevelSequenceWithShotsSettings)
		SLATE_ARGUMENT(UMovieSceneToolsProjectSettings*, ToolsProjectSettings)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();

		FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

		FDetailsViewArgs Details1ViewArgs;
		Details1ViewArgs.bUpdatesFromSelection = false;
		Details1ViewArgs.bLockable = false;
		Details1ViewArgs.bAllowSearch = false;
		Details1ViewArgs.bShowOptions = false;
		Details1ViewArgs.bAllowFavoriteSystem = false;
		Details1ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		Details1ViewArgs.ViewIdentifier = "LevelSequenceWithShotsSettings";

		Details1View = PropertyEditor.CreateDetailView(Details1ViewArgs);

		FDetailsViewArgs Details2ViewArgs;
		Details2ViewArgs.bUpdatesFromSelection = false;
		Details2ViewArgs.bLockable = false;
		Details2ViewArgs.bAllowSearch = false;
		Details2ViewArgs.bShowOptions = false;
		Details2ViewArgs.bAllowFavoriteSystem = false;
		Details2ViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		Details2ViewArgs.ViewIdentifier = "ToolsProjectSettings";

		Details2View = PropertyEditor.CreateDetailView(Details2ViewArgs);

		ChildSlot
		.Padding(8.f)
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(4, 4, 4, 4)
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					Details1View.ToSharedRef()
				]
				+SScrollBox::Slot()
				[
					Details2View.ToSharedRef()
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(10.f)
			[
				SNew(STextBlock)
				.Text(this, &SLevelSequenceWithShotsSettings::GetFullPath)
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SAssignNew(ErrorText, STextBlock)
				.Text(this, &SLevelSequenceWithShotsSettings::GetErrorText)
				.TextStyle(FAppStyle::Get(), TEXT("Log.Warning"))
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(5.f)
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("CreateLevelSequenceWithShots", "Create Level Sequence with Shots"))
				.OnClicked(this, &SLevelSequenceWithShotsSettings::OnLevelCreateSequenceWithShots)
			]
		];

		if (InArgs._LevelSequenceWithShotsSettings)
		{
			SetLevelSequenceWithShotsSettings(InArgs._LevelSequenceWithShotsSettings);
		}
		if (InArgs._ToolsProjectSettings)
		{
			SetToolsProjectSettings(InArgs._ToolsProjectSettings);
		}
	}

	void SetLevelSequenceWithShotsSettings(ULevelSequenceWithShotsSettings* InLevelSequenceWithShotsSettings)
	{
		LevelSequenceWithShotsSettings = InLevelSequenceWithShotsSettings;

		Details1View->SetObject(InLevelSequenceWithShotsSettings);
	}

	void SetToolsProjectSettings(UMovieSceneToolsProjectSettings* InToolsProjectSettings)
	{
		ToolsProjectSettings = InToolsProjectSettings;

		Details2View->SetObject(InToolsProjectSettings);
	}

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
	{
		Collector.AddReferencedObject(LevelSequenceWithShotsSettings);
		Collector.AddReferencedObject(ToolsProjectSettings);
	}
	virtual FString GetReferencerName() const override
	{
		return TEXT("SLevelSequenceWithShotsSettings");
	}

private:

	FText GetFullPath() const
	{
		const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();
		FString FullPath = LevelSequenceSettings->BasePath.Path;
		FullPath /= LevelSequenceSettings->Name;
		FullPath /= LevelSequenceSettings->Name;
		FullPath += LevelSequenceSettings->Suffix;
		FullPath += TEXT(".uasset");
		return FText::FromString(FullPath);
	}

	FText GetErrorText() const
	{			
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();
		FString FullPath = LevelSequenceSettings->BasePath.Path;
		FullPath /= LevelSequenceSettings->Name;
		FullPath /= LevelSequenceSettings->Name;
		FullPath += LevelSequenceSettings->Suffix;
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(FullPath));
		if (AssetData.IsValid())
		{
			return LOCTEXT("LevelSequenceWithShotsExists", "Warning: Level Sequence with Shots Exists");
		}

		return FText::GetEmpty();
	}


	FReply OnLevelCreateSequenceWithShots()
	{
		const ULevelSequenceWithShotsSettings* LevelSequenceSettings = GetDefault<ULevelSequenceWithShotsSettings>();
		
		// Create a sequence with shots
		FString AssetName = LevelSequenceSettings->Name;
		FString PackagePath = LevelSequenceSettings->BasePath.Path;
		PackagePath /= AssetName;
		AssetName += LevelSequenceSettings->Suffix;

		UObject* LevelSequenceWithShotsAsset = LevelSequenceEditorHelpers::CreateLevelSequenceAsset(AssetName, PackagePath);
		if (LevelSequenceWithShotsAsset)
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LevelSequenceWithShotsAsset);
		
			ILevelSequenceEditorModule& LevelSequenceEditorModule = FModuleManager::LoadModuleChecked<ILevelSequenceEditorModule>("LevelSequenceEditor");		
			LevelSequenceEditorModule.OnLevelSequenceWithShotsCreated().Broadcast(LevelSequenceWithShotsAsset);
			
			LevelSequenceWithShotsSettingsWindow.Pin()->RequestDestroyWindow();
		}

		return FReply::Handled();
	}

	TSharedPtr<IDetailsView> Details1View;
	TSharedPtr<IDetailsView> Details2View;
	TSharedPtr<SEditableTextBox> SequenceWithShotsPathText;
	TSharedPtr<STextBlock> ErrorText;
	TObjectPtr<ULevelSequenceWithShotsSettings> LevelSequenceWithShotsSettings;
	TObjectPtr<UMovieSceneToolsProjectSettings> ToolsProjectSettings;
};
	

void LevelSequenceEditorHelpers::OpenLevelSequenceWithShotsDialog(const TSharedRef<FTabManager>& TabManager)
{
	TSharedPtr<SWindow> ExistingWindow = LevelSequenceWithShotsSettingsWindow.Pin();
	if (ExistingWindow.IsValid())
	{
		ExistingWindow->BringToFront();
	}
	else
	{
		ExistingWindow = SNew(SWindow)
			.Title( LOCTEXT("LevelSequenceWithShotsSettingsTitle", "Level Sequence with Shots Settings") )
			.HasCloseButton(true)
			.SupportsMaximize(false)
			.SupportsMinimize(false)
			.ClientSize(FVector2D(600, 600));

		TSharedPtr<SDockTab> OwnerTab = TabManager->GetOwnerTab();
		TSharedPtr<SWindow> RootWindow = OwnerTab.IsValid() ? OwnerTab->GetParentWindow() : TSharedPtr<SWindow>();
		if(RootWindow.IsValid())
		{
			FSlateApplication::Get().AddWindowAsNativeChild(ExistingWindow.ToSharedRef(), RootWindow.ToSharedRef());
		}
		else
		{
			FSlateApplication::Get().AddWindow(ExistingWindow.ToSharedRef());
		}
	}

	ULevelSequenceWithShotsSettings* LevelSequenceWithShotsSettings = GetMutableDefault<ULevelSequenceWithShotsSettings>();
	UMovieSceneToolsProjectSettings* ToolsProjectSettings = GetMutableDefault<UMovieSceneToolsProjectSettings>();

	ExistingWindow->SetContent(
		SNew(SLevelSequenceWithShotsSettings)
		.LevelSequenceWithShotsSettings(LevelSequenceWithShotsSettings)
		.ToolsProjectSettings(ToolsProjectSettings)
	);

	LevelSequenceWithShotsSettingsWindow = ExistingWindow;
}

UObject* LevelSequenceEditorHelpers::CreateLevelSequenceAsset(const FString& AssetName, const FString& PackagePath, UObject* AssetToDuplicate)
{
	IAssetTools& AssetTools = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools").Get();
	UObject* NewAsset = nullptr;
	for (TObjectIterator<UClass> It ; It ; ++It)
	{
		UClass* CurrentClass = *It;
		if (CurrentClass->IsChildOf(UFactory::StaticClass()) && !(CurrentClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			UFactory* Factory = Cast<UFactory>(CurrentClass->GetDefaultObject());
			if (Factory->CanCreateNew() && Factory->ImportPriority >= 0 && Factory->SupportedClass == ULevelSequence::StaticClass())
			{
				if (AssetToDuplicate != nullptr)
				{
					NewAsset = AssetTools.DuplicateAsset(AssetName, PackagePath, AssetToDuplicate);
				}
				else
				{
					NewAsset = AssetTools.CreateAsset(AssetName, PackagePath, ULevelSequence::StaticClass(), Factory);
				}
				break;
			}
		}
	}
	return NewAsset;
}

#undef LOCTEXT_NAMESPACE
