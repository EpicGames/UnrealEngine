// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class FTabManager;

class LevelSequenceEditorHelpers
{
public:

	/** Open dialog for creating a level sequence with shots */
	static void OpenLevelSequenceWithShotsDialog(const TSharedRef<FTabManager>& TabManager);
	
	/** Create a level sequence asset given an asset name and package path */
	static UObject* CreateLevelSequenceAsset(const FString& AssetName, const FString& PackagePath, UObject* AssetToDuplicate = nullptr);
};
