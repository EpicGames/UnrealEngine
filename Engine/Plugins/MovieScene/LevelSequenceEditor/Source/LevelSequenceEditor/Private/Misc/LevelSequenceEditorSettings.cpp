// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LevelSequenceEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelSequenceEditorSettings)

ULevelSequenceEditorSettings::ULevelSequenceEditorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoBindToSimulate = true;
	bAutoBindToPIE      = true;
}

ULevelSequenceWithShotsSettings::ULevelSequenceWithShotsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, Name(TEXT("Sequence"))
	, Suffix(TEXT("Root"))
	, NumShots(5)
{
	BasePath.Path = TEXT("/Game/Cinematics/Sequences");
}

