// Copyright Epic Games, Inc. All Rights Reserved.
#include "Misc/LevelSequenceEditorSpawnRegister.h"
#include "Modules/ModuleManager.h"
#include "ISequencer.h"
#include "LevelEditor.h"
#include "Layers/LayersSubsystem.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieScene.h"
#include "MovieSceneSpawnable.h"
#include "UObject/ObjectSaveContext.h"
#include "Selection.h"
#include "TransformData.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneBindingReferences.h"
#include "UObject/UObjectIterator.h"
#include "MovieSceneCommonHelpers.h"


#define LOCTEXT_NAMESPACE "LevelSequenceEditorSpawnRegister"

/* FLevelSequenceEditorSpawnRegister structors
 *****************************************************************************/

FLevelSequenceEditorSpawnRegister::FLevelSequenceEditorSpawnRegister()
{
	bShouldClearSelectionCache = true;
	bIsEngineCollectingGarbage = false;

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	OnActorSelectionChangedHandle = LevelEditor.OnActorSelectionChanged().AddRaw(this, &FLevelSequenceEditorSpawnRegister::HandleActorSelectionChanged);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FLevelSequenceEditorSpawnRegister::OnObjectsReplaced);

	OnObjectModifiedHandle = FCoreUObjectDelegates::OnObjectModified.AddRaw(this, &FLevelSequenceEditorSpawnRegister::OnObjectModified);
	OnObjectSavedHandle    = FCoreUObjectDelegates::OnObjectPreSave.AddRaw(this, &FLevelSequenceEditorSpawnRegister::OnPreObjectSaved);
#endif

	OnPreGarbageCollectHandle = FCoreUObjectDelegates::GetPreGarbageCollectDelegate().AddRaw(this, &FLevelSequenceEditorSpawnRegister::UpdateIsEngineCollectingGarbage, true);
	OnPostGarbageCollectHandle = FCoreUObjectDelegates::GetPostGarbageCollect().AddRaw(this, &FLevelSequenceEditorSpawnRegister::UpdateIsEngineCollectingGarbage, false);
}


FLevelSequenceEditorSpawnRegister::~FLevelSequenceEditorSpawnRegister()
{
	if (FLevelEditorModule* LevelEditor = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor"))
	{
		LevelEditor->OnActorSelectionChanged().Remove(OnActorSelectionChangedHandle);
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->OnPreSave().RemoveAll(this);
		Sequencer->OnActivateSequence().RemoveAll(this);
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectModified.Remove(OnObjectModifiedHandle);
	FCoreUObjectDelegates::OnObjectPreSave.Remove(OnObjectSavedHandle);
#endif

	FCoreUObjectDelegates::GetPostGarbageCollect().Remove(OnPostGarbageCollectHandle);
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().Remove(OnPreGarbageCollectHandle);
}


/* FLevelSequenceSpawnRegister interface
 *****************************************************************************/

UObject* FLevelSequenceEditorSpawnRegister::SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex/* = 0*/)
{
	TGuardValue<bool> Guard(bShouldClearSelectionCache, false);

	UObject* NewObject = FMovieSceneSpawnRegister::SpawnObject(BindingId, MovieScene, Template, SharedPlaybackState, BindingIndex);
	
	if (AActor* NewActor = Cast<AActor>(NewObject))
	{
		// Mark as replay rewindable so it persists while a replay is going on in PIE.
		NewActor->bReplayRewindable = true;

		if (!TrackedObjects.Find(NewActor))
		{ 
			// Add an entry to the tracked objects map to keep track of this object (so that it can be saved when modified)
			TrackedObjects.Add(NewActor, FTrackedObjectState(Template, BindingId, BindingIndex));
		}

		if (ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>())
		{
			Layers->InitializeNewActorLayers(NewActor);
		}

		// Select the actor if we think it should be selected
		if (SelectedSpawnedObjects.Contains(FMovieSceneSpawnRegisterKey(Template, BindingId, BindingIndex)))
		{
			GEditor->SelectActor(NewActor, true /*bSelected*/, true /*bNotify*/);
		}
	}

	return NewObject;
}


void FLevelSequenceEditorSpawnRegister::PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID)
{
	TGuardValue<bool> Guard(bShouldClearSelectionCache, false);

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

	UMovieSceneSequence*  Sequence      = Sequencer.IsValid() ? Sequencer->GetEvaluationTemplate().GetSequence(TemplateID) : nullptr;
	UObject*              SpawnedObject = FindSpawnedObject(BindingId, TemplateID, BindingIndex).Get();


	if (SpawnedObject)
	{
		const FTrackedObjectState* TrackedState = TrackedObjects.Find(&Object);
		if (TrackedState && TrackedState->bHasBeenModified)
		{
			// SaveDefaultSpawnableState will reset bHasBeenModified to false
			SaveDefaultSpawnableStateImpl(BindingId, BindingIndex, Sequence, SpawnedObject, Sequencer->GetSharedPlaybackState());

			if (Sequence)
			{
				Sequence->MarkPackageDirty();
			}
		}
	}

	// Cache its selection state
	AActor* Actor = Cast<AActor>(&Object);
	if (Actor && GEditor->GetSelectedActors()->IsSelected(Actor))
	{
		SelectedSpawnedObjects.Add(FMovieSceneSpawnRegisterKey(TemplateID, BindingId, BindingIndex));
		GEditor->SelectActor(Actor, false /*bSelected*/, true /*bNotify*/);
	}

	FObjectKey ThisObject(&Object);
	TrackedObjects.Remove(ThisObject);

	FLevelSequenceSpawnRegister::PreDestroyObject(Object, BindingId, BindingIndex, TemplateID);
}

#if WITH_EDITOR
void FLevelSequenceEditorSpawnRegister::SaveDefaultSpawnableState(const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	if (UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(TemplateID))
	{
		if (UObject* Object = FindSpawnedObject(BindingId, TemplateID, BindingIndex).Get())
		{
			SaveDefaultSpawnableStateImpl(BindingId, BindingIndex, Sequence, Object, SharedPlaybackState);
			Sequence->MarkPackageDirty();
		}
	}
}
#endif

void FLevelSequenceEditorSpawnRegister::SaveDefaultSpawnableStateImpl(const FGuid& BindingId, int32 BindingIndex, UMovieSceneSequence* Sequence, UObject* SpawnedObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	IMovieScenePlayer* Player = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);

	FMovieSceneAnimTypeID SpawnablesTypeID = UMovieSceneSpawnablesSystem::GetAnimTypeID();
	auto RestorePredicate = [SpawnablesTypeID](FMovieSceneAnimTypeID TypeID){ return TypeID != SpawnablesTypeID; };

	if (AActor* Actor = Cast<AActor>(SpawnedObject))
	{
		// Restore state on any components
		for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(Actor))
		{
			if (Component)
			{
				Player->PreAnimatedState.RestorePreAnimatedState(*Component, RestorePredicate);
			}
		}
	}

	// Restore state on the object itself
	Player->PreAnimatedState.RestorePreAnimatedState(*SpawnedObject, RestorePredicate);


	// Copy the template
	MovieSceneHelpers::CopyObjectTemplate(Sequence, BindingId ,SpawnedObject, SharedPlaybackState, BindingIndex);

	if (FTrackedObjectState* TrackedState = TrackedObjects.Find(SpawnedObject))
	{
		TrackedState->bHasBeenModified = false;
	}

	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (Sequencer.IsValid())
	{
		Sequencer->RequestInvalidateCachedData();
		Sequencer->RequestEvaluate();
	}
}

/* FLevelSequenceEditorSpawnRegister implementation
 *****************************************************************************/

void FLevelSequenceEditorSpawnRegister::SetSequencer(const TSharedPtr<ISequencer>& Sequencer)
{
	WeakSequencer = Sequencer;
}


/* FLevelSequenceEditorSpawnRegister callbacks
 *****************************************************************************/

void FLevelSequenceEditorSpawnRegister::HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	if (bShouldClearSelectionCache)
	{
		SelectedSpawnedObjects.Reset();
	}
}

void FLevelSequenceEditorSpawnRegister::OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid())
	{
		return;
	}

	for (auto& Pair : Register)
	{
		TWeakObjectPtr<>& WeakObject = Pair.Value.Object;
		UObject* SpawnedObject = WeakObject.Get();
		if (UObject* NewObject = OldToNewInstanceMap.FindRef(SpawnedObject))
		{
			// Reassign the object
			WeakObject = NewObject;

			// It's a spawnable, so ensure it's transient
			NewObject->SetFlags(RF_Transient);

			// Invalidate the binding - it will be resolved if it's ever asked for again
			Sequencer->GetEvaluationState()->Invalidate(Pair.Key.BindingId, Pair.Key.TemplateID);
		}
	}
}

void FLevelSequenceEditorSpawnRegister::OnObjectModified(UObject* ModifiedObject)
{
	// If we are reinstancing then renaming existing objects aside can cause this callback to be called,
	// which can end up with unnecessary template objects getting updated when we close sequencer down
	if (GIsReinstancing || GIsGCingAfterBlueprintCompile || bIsEngineCollectingGarbage)
	{
		return;
	}

	// If the sequence is evaluating, we don't want object modifications to dirty the sequence itself. 
	// For example, this protects against situations where OnObjectModified would be called in response 
	// to the spawnable being attached with AttachToComponent
	TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();
	if (!Sequencer.IsValid() || Sequencer->IsEvaluating())
	{
		return;
	}

	FTrackedObjectState* TrackedState = TrackedObjects.Find(ModifiedObject);
	while (!TrackedState && ModifiedObject)
	{
		TrackedState = TrackedObjects.Find(ModifiedObject);
		ModifiedObject = ModifiedObject->GetOuter();
	}

	// 
	// When the register is cleaning itself up, there's no need to mark the sequence package dirty because spawnables are just getting removed.
	// This scenario can happen if SpawnableA is attached to SpawnableB. When Restore State occurs, SpawnableB is destroyed, which causes 
	// SpawnableA to be marked as modified, resulting in marking the sequencer package dirty without this check. 
	//
	if (TrackedState && !bCleaningUp)
	{
		TrackedState->bHasBeenModified = true;

		UMovieSceneSequence*   OwningSequence = Sequencer->GetEvaluationTemplate().GetSequence(TrackedState->TemplateID);
		if (OwningSequence)
		{
			OwningSequence->MarkPackageDirty();
			SequencesWithModifiedObjects.Add(OwningSequence);
		}
	}
}

void FLevelSequenceEditorSpawnRegister::OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext)
{
	UMovieSceneSequence* SequenceBeingSaved = Cast<UMovieSceneSequence>(Object);
	if (SequenceBeingSaved && SequencesWithModifiedObjects.Contains(SequenceBeingSaved))
	{
		UMovieScene* MovieSceneBeingSaved = SequenceBeingSaved->GetMovieScene();

		// The object being saved is a movie scene sequence that we've tracked as having a modified spawnable in the world.
		// We need to go through all templates in the current sequence that reference this sequence, saving default state
		// for any spawned objects that have been modified.
		TSharedPtr<ISequencer> Sequencer = WeakSequencer.Pin();

		if (Sequencer.IsValid())
		{
			for (const TTuple<FObjectKey, FTrackedObjectState>& Pair : TrackedObjects)
			{
				UObject* SpawnedObject = Pair.Key.ResolveObjectPtr();
				UMovieSceneSequence*  ThisSequence = Sequencer->GetEvaluationTemplate().GetSequence(Pair.Value.TemplateID);

				if (SpawnedObject && ThisSequence == SequenceBeingSaved)
				{
					SaveDefaultSpawnableStateImpl(Pair.Value.ObjectBindingID, Pair.Value.BindingIndex, ThisSequence, SpawnedObject, Sequencer->GetSharedPlaybackState());
				}
			}
		}
	}
}

#if WITH_EDITOR

TValueOrError<FNewSpawnable, FText> FLevelSequenceEditorSpawnRegister::CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory)
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		TValueOrError<FNewSpawnable, FText> Result = MovieSceneObjectSpawner->CreateNewSpawnableType(SourceObject, OwnerMovieScene, ActorFactory);
		if (Result.IsValid())
		{
			return Result;
		}
	}

	return MakeError(LOCTEXT("NoSpawnerFound", "No spawner found to create new spawnable type"));
}

void FLevelSequenceEditorSpawnRegister::SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings)
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		if (MovieSceneObjectSpawner->CanSetupDefaultsForSpawnable(SpawnedObject))
		{
			MovieSceneObjectSpawner->SetupDefaultsForSpawnable(SpawnedObject, Guid, TransformData, Sequencer, Settings);
			return;
		}
	}
}

void FLevelSequenceEditorSpawnRegister::HandleConvertPossessableToSpawnable(UObject* OldObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TOptional<FTransformData>& OutTransformData)
{
	// @TODO: this could probably be handed off to a spawner if we need anything else to be convertible between spawnable/posessable

	AActor* OldActor = Cast<AActor>(OldObject);
	if (OldActor)
	{
		if (OldActor->GetRootComponent())
		{
			OutTransformData.Emplace();
			OutTransformData->Translation = OldActor->GetRootComponent()->GetRelativeLocation();
			OutTransformData->Rotation = OldActor->GetRootComponent()->GetRelativeRotation();
			OutTransformData->Scale = OldActor->GetRootComponent()->GetRelativeScale3D();
		}

		GEditor->SelectActor(OldActor, false, true);
		UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if (World)
		{
			World->EditorDestroyActor(OldActor, true);

			GEditor->BroadcastLevelActorListChanged();
		}
	}
}

bool FLevelSequenceEditorSpawnRegister::CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const
{
	for (TSharedPtr<IMovieSceneObjectSpawner> MovieSceneObjectSpawner : MovieSceneObjectSpawners)
	{
		UObject* ObjectTemplate = Spawnable.GetObjectTemplate();

		if (ObjectTemplate && ObjectTemplate->IsA(MovieSceneObjectSpawner->GetSupportedTemplateType()))
		{
			return MovieSceneObjectSpawner->CanConvertSpawnableToPossessable(Spawnable);
		}
	}

	return false;
}

#endif

void FLevelSequenceEditorSpawnRegister::UpdateIsEngineCollectingGarbage(bool bIsCollectingGarbage)
{
	bIsEngineCollectingGarbage = bIsCollectingGarbage;
}

#undef LOCTEXT_NAMESPACE
