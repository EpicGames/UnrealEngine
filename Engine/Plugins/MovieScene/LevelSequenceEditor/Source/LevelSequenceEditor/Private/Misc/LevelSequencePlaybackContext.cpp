// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/LevelSequencePlaybackContext.h"
#include "Engine/Level.h"
#include "Misc/LevelSequenceEditorSettings.h"
#include "LevelSequence.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Engine/NetDriver.h"


#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "MovieSceneCaptureDialogModule.h"
#include "LevelSequenceEditorModule.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Editor.h"
#include "Selection.h"

#define LOCTEXT_NAMESPACE "LevelSequencePlaybackContext"

class SLevelSequenceContextPicker : public SCompoundWidget
{
public:

	using FContextAndClient = TTuple<UWorld*, IMovieScenePlaybackClient*>;

	DECLARE_DELEGATE_TwoParams(FOnSetPlaybackContextAndClient, UWorld*, IMovieScenePlaybackClient*);

	SLATE_BEGIN_ARGS(SLevelSequenceContextPicker){}

		/** Attribute for retrieving the bound level sequence */
		SLATE_ATTRIBUTE(ULevelSequence*, Owner)

		/** Attribute for retrieving the current context and client */
		SLATE_ATTRIBUTE(FContextAndClient, OnGetPlaybackContextAndClient)

		/** Called when the user explicitly chooses a new context and client */
		SLATE_EVENT(FOnSetPlaybackContextAndClient, OnSetPlaybackContextAndClient)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	TSharedRef<SWidget> BuildWorldPickerMenu();

	static FText GetContextAndClientDescription(const UWorld* Context, const IMovieScenePlaybackClient* Client);
	static FText GetWorldDescription(const UWorld* World);

	FText GetCurrentContextAndClientText() const
	{
		const FContextAndClient CurrentContextAndClient = PlaybackContextAndClientAttribute.Get();
		check(CurrentContextAndClient.Key);
		return GetContextAndClientDescription(CurrentContextAndClient.Key, CurrentContextAndClient.Value);
	}

	const FSlateBrush* GetBorderBrush() const
	{
		const UWorld* CurrentWorld = PlaybackContextAndClientAttribute.Get().Key;
		check(CurrentWorld);

		if (CurrentWorld->WorldType == EWorldType::PIE)
		{
			return GEditor->bIsSimulatingInEditor ? FAppStyle::GetBrush("LevelViewport.StartingSimulateBorder") : FAppStyle::GetBrush("LevelViewport.StartingPlayInEditorBorder");
		}
		else
		{
			return FStyleDefaults::GetNoBrush();
		}
	}

	void ToggleAutoPIE() const
	{
		ULevelSequenceEditorSettings* Settings = GetMutableDefault<ULevelSequenceEditorSettings>();
		Settings->bAutoBindToPIE = !Settings->bAutoBindToPIE;
		Settings->SaveConfig();

		OnSetPlaybackContextAndClientEvent.ExecuteIfBound(nullptr, nullptr);
	}

	bool IsAutoPIEChecked() const
	{
		return GetDefault<ULevelSequenceEditorSettings>()->bAutoBindToPIE;
	}

	void ToggleAutoSimulate() const
	{
		ULevelSequenceEditorSettings* Settings = GetMutableDefault<ULevelSequenceEditorSettings>();
		Settings->bAutoBindToSimulate = !Settings->bAutoBindToSimulate;
		Settings->SaveConfig();

		OnSetPlaybackContextAndClientEvent.ExecuteIfBound(nullptr, nullptr);
	}

	bool IsAutoSimulateChecked() const
	{
		return GetDefault<ULevelSequenceEditorSettings>()->bAutoBindToSimulate;
	}

	void OnSetPlaybackContextAndClient(TWeakObjectPtr<UWorld> InContext, TWeakInterfacePtr<IMovieScenePlaybackClient> InClient)
	{
		if (UWorld* NewContext = InContext.Get())
		{
			IMovieScenePlaybackClient* NewClient = InClient.Get();
			OnSetPlaybackContextAndClientEvent.ExecuteIfBound(NewContext, NewClient);
		}
	}

	bool IsCurrentPlaybackContextAndClient(TWeakObjectPtr<UWorld> InContext, TWeakInterfacePtr<IMovieScenePlaybackClient> InClient)
	{
		FContextAndClient ContextAndClient = PlaybackContextAndClientAttribute.Get();
		return (InContext == ContextAndClient.Key && InClient == ContextAndClient.Value);
	}

private:
	TAttribute<ULevelSequence*> OwnerAttribute;
	TAttribute<FContextAndClient> PlaybackContextAndClientAttribute;
	FOnSetPlaybackContextAndClient OnSetPlaybackContextAndClientEvent;
};

namespace UE
{
namespace MovieScene
{

/**
 * Finds all actors that implement IMovieScenePlaybackClient in the given world, and return those that can preview the given sequence.
 */
static void FindPlaybackClientActors(const UWorld* InWorld, const ULevelSequence* InLevelSequence, TArray<IMovieScenePlaybackClient*>& OutClients)
{
	if (!InWorld)
	{
		return;
	}

	ULevelInstanceSubsystem* LevelInstanceSubsystem = InWorld->GetSubsystem<ULevelInstanceSubsystem>();

	for (const ULevel* Level : InWorld->GetLevels())
	{
		ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem ? LevelInstanceSubsystem->GetOwningLevelInstance(Level) : nullptr;
		if (LevelInstance && !LevelInstance->IsEditing())
		{
			continue;
		}

		for (AActor* Actor : Level->Actors)
		{
			IMovieScenePlaybackClient* PlaybackClient = Cast<IMovieScenePlaybackClient>(Actor);
			if (InLevelSequence && PlaybackClient && PlaybackClient->CanPreview(*InLevelSequence))
			{
				OutClients.Add(PlaybackClient);
			}
		}
	}
}

}
}

FLevelSequencePlaybackContext::FLevelSequencePlaybackContext(ULevelSequence* InLevelSequence)
	: LevelSequence(InLevelSequence)
{
	FEditorDelegates::MapChange.AddRaw(this, &FLevelSequencePlaybackContext::OnMapChange);
	FEditorDelegates::PreBeginPIE.AddRaw(this, &FLevelSequencePlaybackContext::OnPieEvent);
	FEditorDelegates::BeginPIE.AddRaw(this, &FLevelSequencePlaybackContext::OnPieEvent);
	FEditorDelegates::PostPIEStarted.AddRaw(this, &FLevelSequencePlaybackContext::OnPieEvent);
	FEditorDelegates::PrePIEEnded.AddRaw(this, &FLevelSequencePlaybackContext::OnPieEvent);
	FEditorDelegates::EndPIE.AddRaw(this, &FLevelSequencePlaybackContext::OnPieEvent);

	if (GEngine)
	{
		GEngine->OnWorldAdded().AddRaw(this, &FLevelSequencePlaybackContext::OnWorldListChanged);
		GEngine->OnWorldDestroyed().AddRaw(this, &FLevelSequencePlaybackContext::OnWorldListChanged);
	}
}

FLevelSequencePlaybackContext::~FLevelSequencePlaybackContext()
{
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::PreBeginPIE.RemoveAll(this);
	FEditorDelegates::BeginPIE.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::PrePIEEnded.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnWorldAdded().RemoveAll(this);
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}
}

void FLevelSequencePlaybackContext::OnPieEvent(bool)
{
	WeakCurrentContext = nullptr;
}

void FLevelSequencePlaybackContext::OnMapChange(uint32)
{
	WeakCurrentContext = nullptr;
}

void FLevelSequencePlaybackContext::OnWorldListChanged(UWorld*)
{
	WeakCurrentContext = nullptr;
}

ULevelSequence* FLevelSequencePlaybackContext::GetLevelSequence() const
{
	return LevelSequence.Get();
}

UObject* FLevelSequencePlaybackContext::GetPlaybackContext() const
{
	UpdateCachedContextAndClient();

	if (UObject* Client = GetPlaybackClientAsUObject())
	{
		return Client;
	}

	return WeakCurrentContext.Get();
}

UObject* FLevelSequencePlaybackContext::GetPlaybackContextAsObject() const
{
	return GetPlaybackContext();
}

UObject* FLevelSequencePlaybackContext::GetPlaybackClientAsUObject() const
{
	UpdateCachedContextAndClient();
	return WeakCurrentClient.GetObject();
}

ALevelSequenceActor* FLevelSequencePlaybackContext::GetPlaybackClient() const
{
	return Cast<ALevelSequenceActor>(GetPlaybackClientAsUObject());
}

IMovieScenePlaybackClient* FLevelSequencePlaybackContext::GetPlaybackClientAsInterface() const
{
	UpdateCachedContextAndClient();
	return WeakCurrentClient.Get();
}

TArray<UObject*> FLevelSequencePlaybackContext::GetEventContexts() const
{
	TArray<UObject*> Contexts;
	if (UObject* PlaybackContext = GetPlaybackContext())
	{
		if (UWorld* ContextWorld = PlaybackContext->GetWorld())
		{
			ULevelSequencePlayer::GetEventContexts(*ContextWorld, Contexts);
		}
	}
	return Contexts;
}

void FLevelSequencePlaybackContext::OverrideWith(UWorld* InNewContext, IMovieScenePlaybackClient* InNewClient)
{
	// InNewContext may be null to force an auto update
	WeakCurrentContext = InNewContext;
	WeakCurrentClient = InNewClient;
}

TSharedRef<SWidget> FLevelSequencePlaybackContext::BuildWorldPickerCombo()
{
	return SNew(SLevelSequenceContextPicker)
		.Owner(this, &FLevelSequencePlaybackContext::GetLevelSequence)
		.OnGetPlaybackContextAndClient(this, &FLevelSequencePlaybackContext::GetPlaybackContextAndClient)
		.OnSetPlaybackContextAndClient(this, &FLevelSequencePlaybackContext::OverrideWith);
}

FLevelSequencePlaybackContext::FContextAndClient FLevelSequencePlaybackContext::ComputePlaybackContextAndClient(const ULevelSequence* InLevelSequence)
{
	const ULevelSequenceEditorSettings* Settings            = GetDefault<ULevelSequenceEditorSettings>();
	IMovieSceneCaptureDialogModule*     CaptureDialogModule = FModuleManager::GetModulePtr<IMovieSceneCaptureDialogModule>("MovieSceneCaptureDialog");

	// Some plugins may not want us to automatically attempt to bind to the world where it doesn't make sense,
	// such as movie rendering.
	bool bAllowPlaybackContextBinding = true;
	ILevelSequenceEditorModule* LevelSequenceEditorModule = FModuleManager::GetModulePtr<ILevelSequenceEditorModule>("LevelSequenceEditor");
	if (LevelSequenceEditorModule)
	{
		LevelSequenceEditorModule->OnComputePlaybackContext().Broadcast(bAllowPlaybackContextBinding);
	}

	UWorld* RecordingWorld = CaptureDialogModule ? CaptureDialogModule->GetCurrentlyRecordingWorld() : nullptr;

	// Only allow PIE and Simulate worlds if the settings allow them
	const bool bIsSimulatingInEditor = GEditor && GEditor->bIsSimulatingInEditor;
	const bool bIsPIEValid           = (!bIsSimulatingInEditor && Settings->bAutoBindToPIE) || ( bIsSimulatingInEditor && Settings->bAutoBindToSimulate);

	UWorld* EditorWorld = nullptr;

	// Return PIE worlds if there are any
	for (const FWorldContext& Context : GEngine->GetWorldContexts())
	{
		if (Context.WorldType == EWorldType::PIE)
		{
			UWorld* ThisWorld = Context.World();
			const bool bIsServerWorld = (ThisWorld && ThisWorld->GetNetDriver() && ThisWorld->GetNetDriver()->IsServer());
			if (bIsPIEValid && bAllowPlaybackContextBinding && RecordingWorld != ThisWorld && !bIsServerWorld)
			{
				TArray<IMovieScenePlaybackClient*> PlaybackClients;
				UE::MovieScene::FindPlaybackClientActors(ThisWorld, InLevelSequence, PlaybackClients);
				return FContextAndClient(ThisWorld, (PlaybackClients.Num() > 0 ? PlaybackClients[0] : nullptr));
			}
		}
		else if (Context.WorldType == EWorldType::Editor)
		{
			EditorWorld = Context.World();
		}
	}

	if (ensure(EditorWorld))
	{
		// First search selected actors. If one is selected, use that as the client.
		for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
		{
			if (IMovieScenePlaybackClient* Client = Cast<IMovieScenePlaybackClient>(*It))
			{
				return FContextAndClient(EditorWorld, Client);
			}
		}

		// Otherwise, attempt to find one in the world.
		TArray<IMovieScenePlaybackClient*> PlaybackClients;
		UE::MovieScene::FindPlaybackClientActors(EditorWorld, InLevelSequence, PlaybackClients);
		return FContextAndClient(EditorWorld, (PlaybackClients.Num() > 0 ? PlaybackClients[0] : nullptr));
	}

	return FContextAndClient(nullptr, nullptr);
}

void FLevelSequencePlaybackContext::UpdateCachedContextAndClient() const
{
	if (WeakCurrentContext.Get() != nullptr)
	{
		return;
	}

	FContextAndClient ContextAndClient = ComputePlaybackContextAndClient(LevelSequence.Get());
	check(ContextAndClient.Key);
	WeakCurrentContext = ContextAndClient.Key;
	WeakCurrentClient = ContextAndClient.Value;
}

FLevelSequencePlaybackContext::FContextAndClient FLevelSequencePlaybackContext::GetPlaybackContextAndClient() const
{
	UpdateCachedContextAndClient();
	return FContextAndClient(WeakCurrentContext.Get(), WeakCurrentClient.Get());
}

void SLevelSequenceContextPicker::Construct(const FArguments& InArgs)
{
	OwnerAttribute = InArgs._Owner;
	PlaybackContextAndClientAttribute = InArgs._OnGetPlaybackContextAndClient;
	OnSetPlaybackContextAndClientEvent = InArgs._OnSetPlaybackContextAndClient;
	
	check(OwnerAttribute.IsSet());
	check(PlaybackContextAndClientAttribute.IsSet());
	check(OnSetPlaybackContextAndClientEvent.IsBound());

	ChildSlot
	.Padding(0.0f)
	[
		SNew(SBorder)
		.BorderImage(this, &SLevelSequenceContextPicker::GetBorderBrush)
		.Padding(FMargin(4.f, 0.f))
		[
			SNew(SComboButton)
			.ContentPadding(0)
			.ForegroundColor(FSlateColor::UseForeground())
			.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
			.OnGetMenuContent(this, &SLevelSequenceContextPicker::BuildWorldPickerMenu)
			.ToolTipText(FText::Format(LOCTEXT("WorldPickerTextFomrat", "'{0}': The world context and playback client that sequencer should be bound to, and playback within."), GetCurrentContextAndClientText()))
			.ButtonContent()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.World"))
			]
		]
	];
}

FText SLevelSequenceContextPicker::GetContextAndClientDescription(const UWorld* Context, const IMovieScenePlaybackClient* Client)
{
	const FText WorldDescription = GetWorldDescription(Context);
	if (Client)
	{
		const AActor* ClientActor = Cast<AActor>(Client);
		const FString ClientName = ClientActor ? ClientActor->GetActorNameOrLabel() : GetNameSafe(Cast<UObject>(Client));
		return FText::Format(
				LOCTEXT("PlaybackContextDescription", "{0} ({1})"), 
				WorldDescription,
				FText::FromString(ClientName));
	}
	else
	{
		return FText::Format(
				LOCTEXT("PlaybackContextDescriptionNoActor", "{0} (no actor)"), 
				WorldDescription);
	}
}

FText SLevelSequenceContextPicker::GetWorldDescription(const UWorld* World)
{
	FText PostFix;
	if (World->WorldType == EWorldType::PIE)
	{
		switch(World->GetNetMode())
		{
		case NM_Client:
			PostFix = FText::Format(LOCTEXT("ClientPostfixFormat", " (Client {0})"), FText::AsNumber(World->GetOutermost()->GetPIEInstanceID() - 1));
			break;
		case NM_DedicatedServer:
		case NM_ListenServer:
			PostFix = LOCTEXT("ServerPostfix", " (Server)");
			break;
		case NM_Standalone:
			PostFix = GEditor->bIsSimulatingInEditor ? LOCTEXT("SimulateInEditorPostfix", " (Simulate)") : LOCTEXT("PlayInEditorPostfix", " (PIE)");
			break;
		default:
			break;
		}
	}
	else if (World->WorldType == EWorldType::Editor)
	{
		PostFix = LOCTEXT("EditorPostfix", " (Editor)");
	}

	return FText::Format(LOCTEXT("WorldFormat", "{0}{1}"), FText::FromString(World->GetFName().GetPlainNameString()), PostFix);
}

TSharedRef<SWidget> SLevelSequenceContextPicker::BuildWorldPickerMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	ULevelSequence* LevelSequence = OwnerAttribute.Get();

	const ULevelSequenceEditorSettings* Settings = GetDefault<ULevelSequenceEditorSettings>();
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ActorsHeader", "Actors"));
	{
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			UWorld* World = Context.World();
			if (World == nullptr || (Context.WorldType != EWorldType::PIE && Context.WorldType != EWorldType::Editor))
			{
				continue;
			}

			bool bFoundClients = false;
			if (LevelSequence)
			{
				TArray<IMovieScenePlaybackClient*> PlaybackClients;
				UE::MovieScene::FindPlaybackClientActors(World, LevelSequence, PlaybackClients);
				bFoundClients = PlaybackClients.Num() > 0;

				for (IMovieScenePlaybackClient* PlaybackClient : PlaybackClients)
				{
					MenuBuilder.AddMenuEntry(
							GetContextAndClientDescription(World, PlaybackClient),
							FText(),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateSP(
									this, 
									&SLevelSequenceContextPicker::OnSetPlaybackContextAndClient, 
									MakeWeakObjectPtr(World), TWeakInterfacePtr<IMovieScenePlaybackClient>(PlaybackClient)),
								FCanExecuteAction(),
								FIsActionChecked::CreateSP(
									this, 
									&SLevelSequenceContextPicker::IsCurrentPlaybackContextAndClient, 
									MakeWeakObjectPtr(World), TWeakInterfacePtr<IMovieScenePlaybackClient>(PlaybackClient))
								),
							NAME_None,
							EUserInterfaceActionType::RadioButton
							);
				}
			}
			
			if (!bFoundClients)
			{
				MenuBuilder.AddMenuEntry(
						GetContextAndClientDescription(World, nullptr),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateSP(
								this, 
								&SLevelSequenceContextPicker::OnSetPlaybackContextAndClient, 
								MakeWeakObjectPtr(World), TWeakInterfacePtr<IMovieScenePlaybackClient>()),
							FCanExecuteAction(),
							FIsActionChecked::CreateSP(
								this, 
								&SLevelSequenceContextPicker::IsCurrentPlaybackContextAndClient, 
								MakeWeakObjectPtr(World), TWeakInterfacePtr<IMovieScenePlaybackClient>())
							),
						NAME_None,
						EUserInterfaceActionType::RadioButton
						);
			}
		}
	}
	MenuBuilder.EndSection();


	MenuBuilder.BeginSection(NAME_None, LOCTEXT("OptionsHeader", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoBindPIE_Label", "Auto Bind to PIE"),
			LOCTEXT("AutoBindPIE_Tip",   "Automatically binds an active Sequencer window to the current PIE world, if available."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLevelSequenceContextPicker::ToggleAutoPIE),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLevelSequenceContextPicker::IsAutoPIEChecked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoBindSimulate_Label", "Auto Bind to Simulate"),
			LOCTEXT("AutoBindSimulate_Tip",   "Automatically binds an active Sequencer window to the current Simulate world, if available."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SLevelSequenceContextPicker::ToggleAutoSimulate),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SLevelSequenceContextPicker::IsAutoSimulateChecked)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
