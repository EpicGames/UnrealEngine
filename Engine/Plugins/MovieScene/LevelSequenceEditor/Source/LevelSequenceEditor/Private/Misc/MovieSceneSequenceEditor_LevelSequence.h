// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/BlueprintGeneratedClass.h"
#include "LevelSequence.h"
#include "MovieScene.h"
#include "MovieSceneSequenceEditor.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "LevelSequenceDirector.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Modules/ModuleManager.h"

struct FMovieSceneSequenceEditor_LevelSequence : FMovieSceneSequenceEditor
{
	virtual bool CanCreateEvents(UMovieSceneSequence* InSequence) const
	{
		return true;
	}

	virtual bool CanCreateDirectorBlueprint(UMovieSceneSequence* Sequence) const override
	{
#if WITH_EDITOR
		FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
		const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
		TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();
		FClassViewerInitializationOptions ClassViewerOptions = {};

		if (GlobalClassFilter.IsValid())
		{
			return GlobalClassFilter->IsClassAllowed(ClassViewerOptions, ULevelSequenceDirector::StaticClass(), ClassFilterFuncs);
		}
#endif
		return true;
	}

	virtual UBlueprint* GetBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		ULevelSequence* LevelSequence = CastChecked<ULevelSequence>(InSequence);
		return LevelSequence->GetDirectorBlueprint();
	}

	virtual UBlueprint* CreateBlueprintForSequence(UMovieSceneSequence* InSequence) const override
	{
		UBlueprint* Blueprint = GetBlueprintForSequence(InSequence);
		if (!ensureMsgf(!Blueprint, TEXT("Should not call CreateBlueprintForSequence when one already exists")))
		{
			return Blueprint;
		}

		ULevelSequence* LevelSequence = CastChecked<ULevelSequence>(InSequence);

		Blueprint = FKismetEditorUtilities::CreateBlueprint(ULevelSequenceDirector::StaticClass(), InSequence, FName(*LevelSequence->GetDirectorBlueprintName()), BPTYPE_Normal, UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());
		Blueprint->ClearFlags(RF_Standalone);

		LevelSequence->SetDirectorBlueprint(Blueprint);
		return Blueprint;
	}
};
