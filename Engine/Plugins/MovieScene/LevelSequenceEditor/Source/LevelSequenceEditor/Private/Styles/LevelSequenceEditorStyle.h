// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Paths.h"
#include "Brushes/SlateBorderBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"

/**
 * Implements the visual style of the messaging debugger UI.
 */
class FLevelSequenceEditorStyle final
	: public FSlateStyleSet
{
public:

	/** Default constructor. */
	 FLevelSequenceEditorStyle()
		 : FSlateStyleSet("LevelSequenceEditorStyle")
	 {
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);
		const FVector2D Icon48x48(48.0f, 48.0f);

		const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("LevelSequenceEditor"))->GetContentDir();

		SetContentRoot(ContentDir);
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		// tab icons
		Set("LevelSequenceEditor.Tabs.Sequencer", new IMAGE_BRUSH("icon_tab_sequencer_16x", Icon16x16));

		Set("LevelSequenceEditor.PossessNewActor", new IMAGE_BRUSH_SVG("ActorToSequencer", Icon16x16));
		Set("LevelSequenceEditor.PossessNewActor.Small", new IMAGE_BRUSH_SVG("ActorToSequencer", Icon16x16));

		Set("LevelSequenceEditor.CreateNewLevelSequenceInLevel", new IMAGE_BRUSH_SVG("LevelSequence", Icon16x16));
		Set("LevelSequenceEditor.CreateNewLevelSequenceInLevel.Small", new IMAGE_BRUSH_SVG("LevelSequence", Icon16x16));

		Set("LevelSequenceEditor.CreateNewLevelSequenceWithShotsInLevel", new IMAGE_BRUSH_SVG("LevelSequenceWithShots", Icon16x16));
		Set("LevelSequenceEditor.CreateNewLevelSequenceWithShotsInLevel.Small", new IMAGE_BRUSH_SVG("LevelSequenceWithShots", Icon16x16));
		
		Set("LevelSequenceEditor.CinematicViewportPlayMarker", new IMAGE_BRUSH("CinematicViewportPlayMarker", FVector2D(11, 6)));
		Set("LevelSequenceEditor.CinematicViewportRangeStart", new BORDER_BRUSH("CinematicViewportRangeStart", FMargin(1.f,.3f,0.f,.6f)));
		Set("LevelSequenceEditor.CinematicViewportRangeEnd", new BORDER_BRUSH("CinematicViewportRangeEnd", FMargin(0.f,.3f,1.f,.6f)));

		Set("LevelSequenceEditor.CinematicViewportTransportRangeKey", new IMAGE_BRUSH("CinematicViewportTransportRangeKey", FVector2D(7.f, 7.f)));

		Set("LevelSequenceEditor.SaveAs", new IMAGE_BRUSH("Icon_Sequencer_SaveAs_48x", Icon48x48));
		Set("LevelSequenceEditor.ImportFBX", new IMAGE_BRUSH("Icon_Sequencer_ImportFBX_48x", Icon48x48));
		Set("LevelSequenceEditor.ExportFBX", new IMAGE_BRUSH("Icon_Sequencer_ExportFBX_48x", Icon48x48));

		Set("LevelSequenceEditor.Flag", new IMAGE_BRUSH_SVG("Flag", Icon16x16));
		Set("LevelSequenceEditor.ThumbsDown", new IMAGE_BRUSH_SVG("ThumbsDown", Icon16x16));

		Set("FilmOverlay.DefaultThumbnail", new IMAGE_BRUSH("DefaultFilmOverlayThumbnail", FVector2D(36, 24)));

	 	Set("FilmOverlay.Disabled", new IMAGE_BRUSH("FilmOverlay.Disabled", FVector2D(36, 24)));
		Set("FilmOverlay.Grid2x2", new IMAGE_BRUSH("FilmOverlay.2x2Grid", FVector2D(36, 24)));
		Set("FilmOverlay.Grid3x3", new IMAGE_BRUSH("FilmOverlay.3x3Grid", FVector2D(36, 24)));
		Set("FilmOverlay.Crosshair", new IMAGE_BRUSH("FilmOverlay.Crosshair", FVector2D(36, 24)));
		Set("FilmOverlay.Rabatment", new IMAGE_BRUSH("FilmOverlay.Rabatment", FVector2D(36, 24)));
		
	 	Set("FilmOverlay.Disabled.Small", new IMAGE_BRUSH_SVG("FilmOverlay.Disabled_16", Icon16x16));
	 	Set("FilmOverlay.Grid2x2.Small", new IMAGE_BRUSH_SVG("FilmOverlay.2x2Grid_16", Icon16x16));
	 	Set("FilmOverlay.Grid3x3.Small", new IMAGE_BRUSH_SVG("FilmOverlay.3x3Grid_16", Icon16x16));
	 	Set("FilmOverlay.Crosshair.Small", new IMAGE_BRUSH_SVG("FilmOverlay.Crosshair_16", Icon16x16));
	 	Set("FilmOverlay.Rabatment.Small", new IMAGE_BRUSH_SVG("FilmOverlay.Rabatment_16", Icon16x16));

		// Widget Styles
		const FSlateColor NoGoodCheckedColor(FStyleColors::AccentRed);
		const FSlateColor NoGoodHoveredColor(NoGoodCheckedColor.UseSubduedForeground());
		const FSlateColor NoGoodPressedColor(NoGoodCheckedColor.GetSpecifiedColor() * 0.8f);

		FCheckBoxStyle NoGoodBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetForegroundColor(FLinearColor::White)
			.SetCheckedForegroundColor(NoGoodCheckedColor)
			.SetHoveredForegroundColor(NoGoodHoveredColor)
			.SetPressedForegroundColor(NoGoodPressedColor)
			.SetCheckedHoveredForegroundColor(NoGoodHoveredColor)
			.SetCheckedPressedForegroundColor(NoGoodPressedColor)
			.SetBorderBackgroundColor(FLinearColor::Transparent);

		Set("NoGoodWidget", NoGoodBoxStyle);

		const FSlateColor FlaggedCheckedColor(FStyleColors::AccentYellow);
		const FSlateColor FlaggedHoveredColor(FlaggedCheckedColor.UseSubduedForeground());
		const FSlateColor FlaggedPressedColor(FlaggedCheckedColor.GetSpecifiedColor() * 0.8f);

		FCheckBoxStyle FlaggedBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetForegroundColor(FLinearColor::White)
			.SetCheckedForegroundColor(FlaggedCheckedColor)
			.SetHoveredForegroundColor(FlaggedHoveredColor)
			.SetPressedForegroundColor(FlaggedPressedColor)
			.SetCheckedHoveredForegroundColor(FlaggedHoveredColor)
			.SetCheckedPressedForegroundColor(FlaggedPressedColor)
			.SetBorderBackgroundColor(FLinearColor::Transparent);

		Set("FlaggedWidget", FlaggedBoxStyle);

		FCheckBoxStyle ScoreRatingElementStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetForegroundColor(FSlateColor(FLinearColor::White))
			.SetCheckedForegroundColor(FStyleColors::AccentBlue)
			.SetHoveredForegroundColor(FStyleColors::PrimaryHover)
			.SetPressedForegroundColor(FStyleColors::PrimaryPress)
			.SetCheckedHoveredForegroundColor(FStyleColors::PrimaryHover)
			.SetCheckedPressedForegroundColor(FStyleColors::PrimaryPress)
			.SetBorderBackgroundColor(FSlateColor(FLinearColor::Transparent))
			.SetPadding(FMargin(8, 3));;

		Set("ScoreRatingElement", ScoreRatingElementStyle);

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	 }

	 /** Virtual destructor. */
	 virtual ~FLevelSequenceEditorStyle()
	 {
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	 }

	static TSharedRef<FLevelSequenceEditorStyle> Get()
	{
		if (!Singleton.IsValid())
		{
			Singleton = MakeShareable(new FLevelSequenceEditorStyle);
		}
		return Singleton.ToSharedRef();
	}

private:
	static TSharedPtr<FLevelSequenceEditorStyle> Singleton;
};


#undef IMAGE_BRUSH
#undef BOX_BRUSH
#undef BORDER_BRUSH
