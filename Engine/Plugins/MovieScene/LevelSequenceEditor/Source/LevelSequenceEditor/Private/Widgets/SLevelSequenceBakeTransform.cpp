// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSequenceBakeTransform.h"

#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "ISequencer.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/SWindow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SLevelSequenceBakeTransform"

void SLevelSequenceBakeTransform::Construct(const FArguments& InArgs)
{
	check(InArgs._Sequencer);
	check(InArgs._OnBake.IsBound());

	Settings = MakeShared<TStructOnScope<FBakingAnimationKeySettings>>();
	Settings->InitializeAs<FBakingAnimationKeySettings>();
	*Settings = InArgs._Settings;
	//always setting space to be parent as default, since stored space may not be available.
	Sequencer = InArgs._Sequencer;

	FStructureDetailsViewArgs StructureViewArgs;
	StructureViewArgs.bShowObjects = true;
	StructureViewArgs.bShowAssets = true;
	StructureViewArgs.bShowClasses = true;
	StructureViewArgs.bShowInterfaces = true;

	FDetailsViewArgs ViewArgs;
	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	DetailsView = PropertyEditor.CreateStructureDetailView(ViewArgs, StructureViewArgs, TSharedPtr<FStructOnScope>());

	DetailsView->GetDetailsView()->RegisterInstancedCustomPropertyTypeLayout("FrameNumber",
		FOnGetPropertyTypeCustomizationInstance::CreateSP(Sequencer, &ISequencer::MakeFrameNumberDetailsCustomization));
	DetailsView->SetStructureData(Settings);

	ChildSlot
		[
			SNew(SBorder)
				.Visibility(EVisibility::Visible)
				[
					SNew(SVerticalBox)


						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.f, 0.f, 0.f, 0.f)
						[
							DetailsView->GetWidget().ToSharedRef()
						]

						+ SVerticalBox::Slot()
						.AutoHeight()
						.Padding(0.f, 16.f, 0.f, 16.f)
						[
							SNew(SHorizontalBox)

								+ SHorizontalBox::Slot()
								.FillWidth(1.f)
								[
									SNew(SSpacer)
								]

								+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(8.f, 0.f, 0.f, 0.f)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.Text(LOCTEXT("OK", "OK"))
										.OnClicked_Lambda([this, InArgs]()
											{
												FReply Reply = InArgs._OnBake.Execute(*(Settings->Get()));
												CloseDialog();
												return Reply;

											})
								]

							+ SHorizontalBox::Slot()
								.AutoWidth()
								.HAlign(HAlign_Left)
								.VAlign(VAlign_Center)
								.Padding(8.f, 0.f, 16.f, 0.f)
								[
									SNew(SButton)
										.HAlign(HAlign_Center)
										.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
										.Text(LOCTEXT("Cancel", "Cancel"))
										.OnClicked_Lambda([this]()
											{
												CloseDialog();
												return FReply::Handled();
											})
								]
						]
				]
		];
}

class SLevelSequenceBakeTransformDialogWindow : public SWindow
{
};

FReply SLevelSequenceBakeTransform::OpenDialog(bool bModal)
{
	check(!DialogWindow.IsValid());

	const FVector2D CursorPos = FSlateApplication::Get().GetCursorPos();

	TSharedRef<SLevelSequenceBakeTransformDialogWindow> Window = SNew(SLevelSequenceBakeTransformDialogWindow)
		.Title(LOCTEXT("SLevelSequenceBakeTransformWidgetTitle", "Bake Transform"))
		.CreateTitleBar(true)
		.Type(EWindowType::Normal)
		.SizingRule(ESizingRule::Autosized)
		.ScreenPosition(CursorPos)
		.FocusWhenFirstShown(true)
		.ActivationPolicy(EWindowActivationPolicy::FirstShown)
		[
			AsShared()
		];

	Window->SetWidgetToFocusOnActivate(AsShared());

	DialogWindow = Window;

	Window->MoveWindowTo(CursorPos);

	if (bModal)
	{
		GEditor->EditorAddModalWindow(Window);
	}
	else
	{
		FSlateApplication::Get().AddWindow(Window);
	}

	return FReply::Handled();
}

void SLevelSequenceBakeTransform::CloseDialog()
{
	if (DialogWindow.IsValid())
	{
		DialogWindow.Pin()->RequestDestroyWindow();
		DialogWindow.Reset();
	}
}

#undef LOCTEXT_NAMESPACE