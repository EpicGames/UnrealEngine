// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SLevelSequenceFavoriteRating.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "MetaData/MovieSceneShotMetaData.h"
#include "ScopedTransaction.h"
#include "Styles/LevelSequenceEditorStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/SScoreRating.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SLevelSequenceFavoriteRating"

void SLevelSequenceFavoriteRating::Construct(const FArguments& InArgs)
{
	WeakLevelSequence = InArgs._LevelSequence;

	ChildSlot
		[
			SNew(SHorizontalBox)

				// Thumbs down
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SCheckBox)
							.Padding(FMargin(0.0, 0.0, 8.0, 0.0))
							.Style(&FLevelSequenceEditorStyle::Get()->GetWidgetStyle<FCheckBoxStyle>("NoGoodWidget"))
							.ToolTipText(LOCTEXT("MarkLevelSequenceAsNoGoodTooltip", "Mark as No Good"))
							.IsChecked_Lambda([&WeakLevelSequence = this->WeakLevelSequence]()->ECheckBoxState
								{
									bool bValue = false;
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										if (UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindMetaData<UMovieSceneShotMetaData>())
										{
											bValue = MetaData->GetIsNoGood().IsSet() ? MetaData->GetIsNoGood().GetValue() : false;
										}
									}
									return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([&WeakLevelSequence = this->WeakLevelSequence](ECheckBoxState State)
								{
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindOrAddMetaData<UMovieSceneShotMetaData>();
										const bool bIsNoGood = State == ECheckBoxState::Checked;
										const FText TransactionSessionName = bIsNoGood
											? LOCTEXT("SetLevelSequenceIsNoGoodTransaction", "Set Level Sequence Is No Good")
											: LOCTEXT("UnsetLevelSequenceIsNoGoodTransaction", "Unset Level Sequence Is No Good");

										const FScopedTransaction Transaction(TransactionSessionName);
										MetaData->SetFlags(RF_Transactional);
										MetaData->Modify();
										MetaData->SetIsNoGood(bIsNoGood);

										if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
										{
											AssetRegistry->AssetUpdateTags(StrongLevelSequence.Get(), EAssetRegistryTagsCaller::FullUpdate);
										}
									}
								}))
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.ThumbsDown"))
							]
					]

				// Flagged
				+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SCheckBox)
							.Padding(FMargin(8.0, 0.0, 16.0, 0.0))
							.Style(&FLevelSequenceEditorStyle::Get()->GetWidgetStyle<FCheckBoxStyle>("FlaggedWidget"))
							.ToolTipText(LOCTEXT("FlagLevelSequenceTooltip", "Flag"))
							.IsChecked_Lambda([&WeakLevelSequence = this->WeakLevelSequence]()->ECheckBoxState
								{
									bool bValue = false;
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										if (UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindMetaData<UMovieSceneShotMetaData>())
										{
											bValue = MetaData->GetIsFlagged().IsSet() ? MetaData->GetIsFlagged().GetValue() : false;
										}
									}
									return bValue ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([&WeakLevelSequence = this->WeakLevelSequence](ECheckBoxState State)
								{
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindOrAddMetaData<UMovieSceneShotMetaData>();
										const bool bIsFlagged = State == ECheckBoxState::Checked;
										const FText TransactionSessionName = bIsFlagged 
											? LOCTEXT("SetLevelSequenceIsFlaggedTransaction", "Set Level Sequence Is Flagged")
											: LOCTEXT("UnsetLevelSequenceIsFlaggedTransaction", "Unset Level Sequence Is Flagged");

										const FScopedTransaction Transaction(TransactionSessionName);
										MetaData->SetFlags(RF_Transactional);
										MetaData->Modify();
										MetaData->SetIsFlagged(bIsFlagged);

										if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
										{
											AssetRegistry->AssetUpdateTags(StrongLevelSequence.Get(), EAssetRegistryTagsCaller::FullUpdate);
										}
									}
								}))
							[
								SNew(SImage)
									.ColorAndOpacity(FSlateColor::UseForeground())
									.Image(FLevelSequenceEditorStyle::Get()->GetBrush("LevelSequenceEditor.Flag"))
							]
					]

				// Rating
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					[
						SNew(SScoreRating)
							.MaxScore(3)
							.OnScoreChanged_Lambda([&WeakLevelSequence = this->WeakLevelSequence](int32 NewScore)
								{
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindOrAddMetaData<UMovieSceneShotMetaData>();
										const FScopedTransaction Transaction(LOCTEXT("SetLevelSequenceFavoriteRatingTransaction", "Set Level Sequence Favorite Rating"));
										MetaData->SetFlags(RF_Transactional);
										MetaData->Modify();
										if (NewScore > 0)
										{
											MetaData->SetFavoriteRating(NewScore);
										}
										else
										{
											MetaData->ClearFavoriteRating();
										}

										if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
										{
											AssetRegistry->AssetUpdateTags(StrongLevelSequence.Get(), EAssetRegistryTagsCaller::FullUpdate);
										}
									}
								})
							.Score_Lambda([&WeakLevelSequence = this->WeakLevelSequence]() -> int32
								{
									if (TStrongObjectPtr<ULevelSequence> StrongLevelSequence = WeakLevelSequence.Pin())
									{
										if (UMovieSceneShotMetaData* MetaData = StrongLevelSequence->FindMetaData<UMovieSceneShotMetaData>();
											MetaData != nullptr && MetaData->GetFavoriteRating().IsSet())
										{
											return MetaData->GetFavoriteRating().GetValue();
										}
									}
									return 0;
								})

					]
		];
}

#undef LOCTEXT_NAMESPACE
