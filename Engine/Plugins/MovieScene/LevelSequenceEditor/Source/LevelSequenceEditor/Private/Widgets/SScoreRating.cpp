// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SScoreRating.h"
#include "Widgets/SScoreRatingElement.h"

#define LOCTEXT_NAMESPACE "SScoreRating"

SLATE_IMPLEMENT_WIDGET(SScoreRating)
void SScoreRating::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ElementIconAttribute", ElementIconAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ScoreAttribute", ScoreAttribute, EInvalidateWidgetReason::Layout);
}

SScoreRating::SScoreRating() :
	ElementIconAttribute(*this),
	ScoreAttribute(*this)
{

}

void SScoreRating::Construct(const FArguments& InArgs)
{
	LayoutBox = SNew(SHorizontalBox);
	// Ensure we always have at least 2 and no more than 10
	MaxScore = FMath::Clamp(InArgs._MaxScore, 2, 10);
	OnScoreChanged = InArgs._OnScoreChanged;
	ScoreAttribute.Assign(*this, InArgs._Score);
	ElementIconAttribute.Assign(*this, InArgs._ElementIcon);

	for (int32 ScoreIndex = 0; ScoreIndex < MaxScore; ++ScoreIndex)
	{
		LayoutBox->AddSlot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.0, 4.0, 0.0))
			[
				SNew(SScoreRatingElement)
					.Icon(ElementIconAttribute.Get())
					.ToolTipText(FText::Format(LOCTEXT("SetScoreRatingTooltip", "Score as {0}"), ScoreIndex + 1))
					.IsChecked_Lambda([&Score = this->ScoreAttribute, &HoverIndex = this->HoverIndex, ScoreIndex]() -> bool
						{
							return  ScoreIndex < HoverIndex || Score.Get() > ScoreIndex;
						})
					.OnCheckStateChanged(FOnBinaryCheckStateChanged::CreateLambda([this, ScoreIndex](bool bIsChecked)
						{
							const int32 NewScore = ScoreIndex + 1;
							NotifyScoreChanged(NewScore != ScoreAttribute.Get() ? NewScore : 0);
						}))
					.OnHoverStarted_Raw(this, &SScoreRating::SetHoverIndex, ScoreIndex)
					.OnHoverFinished_Raw(this, &SScoreRating::SetHoverIndex, static_cast<int32>(INDEX_NONE))
			];
	}

	ChildSlot
		[
			LayoutBox.ToSharedRef()
		];
}

void SScoreRating::NotifyScoreChanged(int32 NewScore)
{
	OnScoreChanged.ExecuteIfBound(NewScore);
}

#undef LOCTEXT_NAMESPACE
