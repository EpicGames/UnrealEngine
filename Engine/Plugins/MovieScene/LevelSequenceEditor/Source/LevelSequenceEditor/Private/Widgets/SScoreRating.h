// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"

/** Delegate that is executed when the score is changed */
DECLARE_DELEGATE_OneParam(FOnScoreChanged, int32 )

/** Widget for viewing and setting a score rating. */
class SScoreRating : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SScoreRating, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SScoreRating)
		: _MaxScore(3)
		, _ElementIcon(FAppStyle::GetBrush("Icons.Star"))
		, _OnScoreChanged()
		, _Score(0)
		{}

		/** The max score shown by the widget */
		SLATE_ARGUMENT( int32, MaxScore )

		/** The icon brush to use for each score rating element */
		SLATE_ATTRIBUTE( const FSlateBrush*, ElementIcon )

		/** Event triggered when the score is changed. */
		SLATE_EVENT( FOnScoreChanged, OnScoreChanged)

		/** The current score */
		SLATE_ATTRIBUTE( int32, Score )
	SLATE_END_ARGS()

	SScoreRating();

	void Construct(const FArguments& InArgs);

private:
	/** Set the current hover index */
	inline void SetHoverIndex(int32 Index) { HoverIndex = Index; }

	/** Notify the score has changed with the new score */
	void NotifyScoreChanged(int32 NewScore);

	/** The overall layout widget */
	TSharedPtr<SHorizontalBox> LayoutBox;

	/** Max possible score to display */
	int32 MaxScore = 3;

	/** Event triggered when the score is changed. */
	FOnScoreChanged OnScoreChanged;

	/** Current score rating index that the mouse is hovering over */
	int32 HoverIndex = 0;

	/** The slate brush to use to draw the element icons. */
	TSlateAttribute<const FSlateBrush*> ElementIconAttribute;

	/** The current score */
	TSlateAttribute<int32> ScoreAttribute;
};
