// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SScoreRatingElement.h"
#include "Styles/LevelSequenceEditorStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SScoreRatingElement"

SLATE_IMPLEMENT_WIDGET(SScoreRatingElement)
void SScoreRatingElement::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IconAttribute", IconAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "IsCheckedAttribute", IsCheckedAttribute, EInvalidateWidgetReason::Layout);
}

SScoreRatingElement::SScoreRatingElement() :
	IconAttribute(*this),
	IsCheckedAttribute(*this)
{

}

void SScoreRatingElement::Construct(const FArguments& InArgs)
{
	OnHoverStarted = InArgs._OnHoverStarted;
	OnHoverFinished = InArgs._OnHoverFinished;
	OnCheckStateChanged = InArgs._OnCheckStateChanged;
	IconAttribute.Assign(*this, InArgs._Icon);
	IsCheckedAttribute.Assign(*this, InArgs._IsChecked);

	ChildSlot
		[
			SNew(SCheckBox)
				.Padding(FMargin(0.0f))
				.Style(&FLevelSequenceEditorStyle::Get()->GetWidgetStyle<FCheckBoxStyle>("ScoreRatingElement"))
				.IsChecked_Lambda([&IsChecked = this->IsCheckedAttribute]() -> ECheckBoxState { return IsChecked.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged(FOnCheckStateChanged::CreateLambda([&OnCheckStateChangedRef = this->OnCheckStateChanged](ECheckBoxState State) 
					{
						OnCheckStateChangedRef.ExecuteIfBound(State == ECheckBoxState::Checked);
					}))
				[
					SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(IconAttribute.Get())
				]
		];
}

void SScoreRatingElement::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseEnter(MyGeometry, MouseEvent);
	OnHoverStarted.ExecuteIfBound();
}

void SScoreRatingElement::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	SCompoundWidget::OnMouseLeave(MouseEvent);
	OnHoverFinished.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
