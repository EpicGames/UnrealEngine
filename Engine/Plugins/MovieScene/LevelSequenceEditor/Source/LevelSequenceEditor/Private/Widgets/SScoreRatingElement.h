// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

/** Delegate that is executed when a Score Rating Element hover event happens */
DECLARE_DELEGATE(FOnScoreRatingElementHoverEvent);

/** Delegate that is executed when a binary check state change occurs, either to/from true/false */
DECLARE_DELEGATE_OneParam(FOnBinaryCheckStateChanged, bool);

/** 
 * Specialized widget for a score value to expose when mouse hover events occur.
 * Ideally this would be contained entirely in SScoreRating however the need to get the hover events
 * mean we require an extra wrapper class.
 */
class SScoreRatingElement : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SScoreRatingElement, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SScoreRatingElement)
		: _OnHoverStarted()
		, _OnHoverFinished()
		, _OnCheckStateChanged()
		, _Icon(FAppStyle::GetBrush("Icons.Star"))
		, _IsChecked(false)
		{
		}

		/** Called when a hover event starts */
		SLATE_EVENT( FOnScoreRatingElementHoverEvent, OnHoverStarted )

		/** Called when a hover event finishes */
		SLATE_EVENT( FOnScoreRatingElementHoverEvent, OnHoverFinished )
		
		/** Called when a a state change occurs on the Rating Star through user input */
		SLATE_EVENT( FOnBinaryCheckStateChanged, OnCheckStateChanged )

		/** The Slate brush to use for this element */
		SLATE_ATTRIBUTE( const FSlateBrush*, Icon )

		/** Whether the Score Rating Element is currently in its checked state. */
		SLATE_ATTRIBUTE( bool, IsChecked )

	SLATE_END_ARGS()

	SScoreRatingElement();

	void Construct(const FArguments& InArgs);

	// SWidget Interface
	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void OnMouseLeave(const FPointerEvent& MouseEvent) override;

private:
	/** Delegate fired when a hover event starts */
	FOnScoreRatingElementHoverEvent OnHoverStarted;

	/** Delegate fired when a hover event finishes */
	FOnScoreRatingElementHoverEvent OnHoverFinished;

	/** Delegate fired when a state change event occurs */
	FOnBinaryCheckStateChanged OnCheckStateChanged;

	/** The slate brush to use to draw the icon. */
	TSlateAttribute<const FSlateBrush*> IconAttribute;

	/** Are we checked */
	TSlateAttribute<bool> IsCheckedAttribute;
};
