// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequence.h"
#include "AssetDefinitionDefault.h"

#include "AssetDefinition_LevelSequence.generated.h"

#define UE_API LEVELSEQUENCEEDITOR_API

UCLASS(MinimalAPI)
class UAssetDefinition_LevelSequence : public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	// UAssetDefinition Begin
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("AssetTypeActions", "AssetTypeActions_LevelSequence", "Level Sequence"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(200, 80, 80)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return ULevelSequence::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Basic, EAssetCategoryPaths::Cinematics };
		return Categories;
	}
	UE_API virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const override;
	UE_API virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const override;
	// UAssetDefinition End
};

#undef UE_API
