// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "FilmOverlayToolkit.generated.h"

#define UE_API LEVELSEQUENCEEDITOR_API

struct IFilmOverlay;

/** Tool kit for film overlays */
UCLASS(MinimalAPI)
class UFilmOverlayToolkit : public UObject
{
public:

	GENERATED_BODY()

	/** Register a primary film overlay */
	static UE_API void RegisterPrimaryFilmOverlay(const FName& FilmOverlayName, TSharedPtr<IFilmOverlay> FilmOverlay);
	
	/** Unregister a primary film overlay */
	static UE_API void UnregisterPrimaryFilmOverlay(const FName& FilmOverlayName);

	/** Get the primary film overlays */
	static UE_API const TMap<FName, TSharedPtr<IFilmOverlay>>& GetPrimaryFilmOverlays();

	/** Register a toggleable film overlay */
	static UE_API void RegisterToggleableFilmOverlay(const FName& FilmOverlayName, TSharedPtr<IFilmOverlay> FilmOverlay);

	/** Unregister a toggleable film overlay */
	static UE_API void UnregisterToggleableFilmOverlay(const FName& FilmOverlayName);

	/** Get the toggleable film overlays */
	static UE_API const TMap<FName, TSharedPtr<IFilmOverlay>>& GetToggleableFilmOverlays();

private:

	/** The primary film overlays (only one can be active at a time) */
	static UE_API TMap<FName, TSharedPtr<IFilmOverlay>> PrimaryFilmOverlays;

	/** The toggleable film overlays (any number can be active at a time) */
	static UE_API TMap<FName, TSharedPtr<IFilmOverlay>> ToggleableFilmOverlays;
};

#undef UE_API
