// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Geometry.h"
#include "Math/Color.h"
#include "Styling/SlateBrush.h"
#include "Widgets/SWidget.h"

/** Abstract base class that defines how an overlay should be drawn over the viewport */
struct IFilmOverlay
{
	IFilmOverlay()
		: Tint(FLinearColor::White) 
		, bEnabled(false) {}
		
	virtual ~IFilmOverlay() = default;

	/** Get a localized display name that is representative of this overlay */
	virtual FText GetDisplayName() const = 0;

	/** Get a localized tooltip for this overlay */
	virtual FText GetToolTip() const { return GetDisplayName(); }

	/** Get a slate thumbnail brush that is representative of this overlay. 36x24 recommended */
	virtual const FSlateBrush* GetThumbnail() const = 0;

	/** Construct a widget that controls this overlay's arbitrary settings. Only used for toggleable overlays. */
	virtual TSharedPtr<SWidget> ConstructSettingsWidget() { return nullptr; }

	/** Paint the overlay */
	virtual void Paint(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const = 0;

public:

	/** Get/Set a custom tint for this overlay */
	const FLinearColor& GetTint() const { return Tint; }
	void SetTint(const FLinearColor& InTint) { Tint = InTint; }

	/** Get/Set whether this overlay should be drawn */
	bool IsEnabled() const { return bEnabled; }
	void SetEnabled(bool bInEnabled) { bEnabled = bInEnabled; }

protected:

	/** Tint to apply to this overlay */
	FLinearColor Tint;

	/** Whether this overlay is enabled or not */
	bool bEnabled;
};
