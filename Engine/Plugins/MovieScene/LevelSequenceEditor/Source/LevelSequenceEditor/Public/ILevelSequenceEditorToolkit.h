// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Toolkits/AssetEditorToolkit.h"

class ISequencer;

/**
 * Implements an Editor toolkit for level sequences.
 */
class ILevelSequenceEditorToolkit
	: public FAssetEditorToolkit
{
public:

	/**
	 * Access the sequencer that is displayed on this asset editor UI
	 */
	virtual TSharedPtr<ISequencer> GetSequencer() const = 0;
};
