// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneTimeUnit.h"
#include "SequencerCurveEditorObject.h"
#include "SequencerSettings.h"
#include "LevelSequenceEditorBlueprintLibrary.generated.h"

#define UE_API LEVELSEQUENCEEDITOR_API

struct FMovieSceneBindingProxy;
struct FMovieSceneObjectBindingID;
struct FMovieSceneSequencePlaybackParams;

class ISequencer;
class ULevelSequence;
class UMovieSceneFolder;
class UMovieSceneSection;
class UMovieSceneSubSection;
class UMovieSceneTrack;


UCLASS(MinimalAPI)
class ULevelSequenceEditorBlueprintLibrary : public UBlueprintFunctionLibrary
{
public:

	GENERATED_BODY()

	/*
	 * Open a level sequence asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API bool OpenLevelSequence(ULevelSequence* LevelSequence);

	/*
	 * Get the currently opened root level sequence asset
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API ULevelSequence* GetCurrentLevelSequence();

	/*
	 * Get the currently focused/viewed level sequence asset if there is a hierarchy of sequences.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API ULevelSequence* GetFocusedLevelSequence();

	/*
	 * Focus/view the sequence associated to the given sub sequence section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void FocusLevelSequence(UMovieSceneSubSection* SubSection);

	/*
	 * Focus/view the parent sequence, popping out of the current sub sequence section.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void FocusParentSequence();

	/*
	 * Get the current sub section hierarchy from the current sequence to the section associated with the focused sequence.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API TArray<UMovieSceneSubSection*> GetSubSequenceHierarchy();
	
	/*
	 * Close
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void CloseLevelSequence();

	/**
	 * Play the current level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void Play();

	/**
	 * Pause the current level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void Pause();

public:

	UE_DEPRECATED(5.4, "Use SetCurrentTime that takes a FMovieSceneSequencePlaybackParams")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", meta = (DeprecatedFunction, DeprecationMessage = "Use SetCurrentTime that takes a FMovieSceneSequencePlaybackParams"))
	static UE_API void SetCurrentTime(int32 NewFrame);
	
	UE_DEPRECATED(5.4, "Use GetCurrentTime that returns a FMovieSceneSequencePlaybackParams")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", meta = (DeprecatedFunction, DeprecationMessage = "Use GetCurrentTime that returns a FMovieSceneSequencePlaybackParams"))
	static UE_API int32 GetCurrentTime();

	UE_DEPRECATED(5.4, "Use SetCurrentLocalTime that takes a FMovieSceneSequencePlaybackParams")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", meta = (DeprecatedFunction, DeprecationMessage = "Use SetCurrentLocalTime that takes a FMovieSceneSequencePlaybackParams"))
	static UE_API void SetCurrentLocalTime(int32 NewFrame);

	UE_DEPRECATED(5.4, "Use GetCurrentLocalTime that takes a FMovieSceneSequencePlaybackParams")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", meta = (DeprecatedFunction, DeprecationMessage = "Use GetCurrentLocalTime that returns a FMovieSceneSequencePlaybackParams"))
	static UE_API int32 GetCurrentLocalTime();

	/**
	 * Set global playhead position for the current level sequence. If the requested time is the same as the current time, an evaluation will be forced.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Set Current Time")
	static UE_API void SetGlobalPosition(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	 * Get the current global playhead position
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Get Current Time")
	static UE_API FMovieSceneSequencePlaybackParams GetGlobalPosition(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	 * Set local playhead position for the current level sequence. If the requested time is the same as the current time, an evaluation will be forced.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Set Current Local Time")
	static UE_API void SetLocalPosition(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	 * Get the current local playhead position
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Get Current Local Time")
	static UE_API FMovieSceneSequencePlaybackParams GetLocalPosition(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/**
	 * Set playback speed of the current level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetPlaybackSpeed(float NewPlaybackSpeed);

	/**
	 * Get playback speed of the current level sequence
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API float GetPlaybackSpeed();

	/**
     * Set loop mode (note this is a per user preference)
     */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetLoopMode(ESequencerLoopMode NewLoopMode);

	/**
	 * Get loop mode (note this is a per user preference)
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API ESequencerLoopMode GetLoopMode();

	/**
	 * Play from the current time to the requested time in frames
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void PlayTo(FMovieSceneSequencePlaybackParams PlaybackParams, EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

public:

	/** Return the playback start position */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Get Playback Start Time")
	static UE_API FMovieSceneSequencePlaybackParams GetPlaybackStartPosition(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Return end of the playback range in the Sequencer UI, which accounts for the exclusive upper bound */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor", DisplayName = "Get Playback End Time")
	static UE_API FMovieSceneSequencePlaybackParams GetPlaybackEndPosition(EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate);

	/** Check whether the sequence is actively playing. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API bool IsPlaying();

public:

	/** Gets the currently selected tracks. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<UMovieSceneTrack*> GetSelectedTracks();

	/** Gets the currently selected sections. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<UMovieSceneSection*> GetSelectedSections();

	/** Gets the currently selected channels. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<FSequencerChannelProxy> GetSelectedChannels();

	/** Gets the channel with selected keys. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<FSequencerChannelProxy> GetChannelsWithSelectedKeys();

	/** Gets the selected key indices with this channel */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<int32> GetSelectedKeys(const FSequencerChannelProxy& ChannelProxy);

	/** Gets the currently selected folders. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<UMovieSceneFolder*> GetSelectedFolders();

	/** Gets the currently selected object bindings */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<FMovieSceneBindingProxy> GetSelectedBindings();

	/** Select tracks */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectTracks(const TArray<UMovieSceneTrack*>& Tracks);

	/** Select sections */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectSections(const TArray<UMovieSceneSection*>& Sections);

	/** Select channels */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectChannels(const TArray<FSequencerChannelProxy>& Channels);

	/** Select keys from indices */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectKeys(const FSequencerChannelProxy& Channel, const TArray<int32>& Indices);

	/** Select folders */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectFolders(const TArray<UMovieSceneFolder*>& Folders);

	/** Select bindings */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SelectBindings(const TArray<FMovieSceneBindingProxy>& ObjectBindings);

	/** Deselect bindings */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void DeselectBindings(const TArray<FMovieSceneBindingProxy>& ObjectBindings);

	/** Empties the current selection. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void EmptySelection();

	/** Set the selection range start frame. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetSelectionRangeStart(int32 NewFrame);

	/** Set the selection range end frame. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetSelectionRangeEnd(int32 NewFrame);

	/** Get the selection range start frame. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API int32 GetSelectionRangeStart();

	/** Get the selection range end frame. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API int32 GetSelectionRangeEnd();

public:

	/** Refresh Sequencer UI on next tick */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void RefreshCurrentLevelSequence();

	/** Force sequencer evaluation and UI update immediately */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void ForceUpdate();

	/** Get the object bound to the given binding ID with the current level sequence editor */
	UFUNCTION(BlueprintPure, Category="Level Sequence Editor")
	static UE_API TArray<UObject*> GetBoundObjects(FMovieSceneObjectBindingID ObjectBinding);

	/** Check whether the current level sequence and its descendants are locked for editing. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API bool IsLevelSequenceLocked();

	/** Sets the lock for the current level sequence and its descendants for editing. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetLockLevelSequence(bool bLock);

public:

	/** Check whether the lock for the viewport to the camera cuts is enabled. */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API bool IsCameraCutLockedToViewport();

	/** Sets the lock for the viewport to the camera cuts. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetLockCameraCutToViewport(bool bLock);

public:

	/** Gets whether the specified track filter is on/off */
	UE_DEPRECATED(5.5, "Use IsTrackFilterActive")
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API bool IsTrackFilterEnabled(const FText& TrackFilterName);

	/** Gets whether the specified track filter is on/off */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API bool IsTrackFilterActive(const FText& TrackFilterName);

	/** Sets the specified track filter to be on or off */
	UE_DEPRECATED(5.5, "Use SetTrackFilterActive")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetTrackFilterEnabled(const FText& TrackFilterName, bool bEnabled);

	/** Sets the specified track filter to be on or off */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	static UE_API void SetTrackFilterActive(const FText& TrackFilterName, bool bActive);

	/** Gets all the available track filter names */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	static UE_API TArray<FText> GetTrackFilterNames();

public:

	/** Get if a custom color for specified channel idendified by it's class and identifier exists */
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::HasCustomColorForChannel")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::HasCustomColorForChannel"))
	static UE_API bool HasCustomColorForChannel(UClass* Class, const FString& Identifier);
	
	/** Get custom color for specified channel idendified by it's class and identifier,if none exists will return white*/
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::GetCustomColorForChannel")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::HasCustomColorForChannel"))
	static UE_API FLinearColor GetCustomColorForChannel(UClass* Class, const FString& Identifier);
	
	/** Set Custom Color for specified channel idendified by it's class and identifier. This will be stored in editor user preferences.*/
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::SetCustomColorForChannel")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::SetCustomColorForChannel"))
	static UE_API void SetCustomColorForChannel(UClass* Class, const FString& Identifier, const FLinearColor& NewColor);
	
	/** Set Custom Color for specified channels idendified by it's class and identifiers. This will be stored in editor user preferences.*/
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::DeleteColorForChannels")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::DeleteColorForChannels"))
	static UE_API void SetCustomColorForChannels(UClass* Class, const TArray<FString>& Identifiers, const TArray<FLinearColor>& NewColors);
	
	/** Set Random Colors for specified channels idendified by it's class and identifiers. This will be stored in editor user preferences.*/
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::SetRandomColorForChannels")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::SetRandomColorForChannels"))
	static UE_API void SetRandomColorForChannels(UClass* Class, const TArray<FString>& Identifiers);
	
	/** Delete for specified channel idendified by it's class and identifier.*/
	UE_DEPRECATED(5.4, "Use USequencerCurveEditorObject::DeleteColorForChannels")
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor",
	meta = (DeprecatedFunction, DeprecationMessage = "Use USequencerCurveEditorObject::DeleteColorForChannels"))
	static UE_API void DeleteColorForChannels(UClass* Class, FString& Identifier);

public:

	/*
	 * Callbacks
	 */

public:

	/**
	 * Internal function to assign a sequencer singleton.
	 * NOTE: Only to be called by LevelSequenceEditor::Construct.
	 */
	static UE_API void SetSequencer(TSharedRef<ISequencer> InSequencer);
};

#undef UE_API
