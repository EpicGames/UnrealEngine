// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SequencerCustomizationManager.h"
#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"


class UObject;


/**
 * Interface for the Level Sequence Editor module.
 */
class ILevelSequenceEditorModule
	: public IModuleInterface
{
public:
	DECLARE_EVENT_OneParam(ILevelSequenceEditorModule, FOnLevelSequenceWithShotsCreated, UObject*);
	virtual FOnLevelSequenceWithShotsCreated& OnLevelSequenceWithShotsCreated() = 0;

	DECLARE_MULTICAST_DELEGATE_OneParam(FAllowPlaybackContext, bool&);
	virtual FAllowPlaybackContext& OnComputePlaybackContext() = 0;

	virtual FGuid RegisterAdditionalLevelSequenceEditorCustomization(FOnGetSequencerCustomizationInstance OnGetInstance) = 0;
	virtual void UnregisterAdditionalLevelSequenceEditorCustomization(FGuid InHandle) = 0;
};
