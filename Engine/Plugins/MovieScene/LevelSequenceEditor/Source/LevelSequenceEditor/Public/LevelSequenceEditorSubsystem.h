// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "EditorSubsystem.h"

#include "MovieSceneTimeUnit.h"
#include "Containers/SortedMap.h"
#include "UObject/StructOnScope.h"
#include "UniversalObjectLocator.h"
#include "UniversalObjectLocatorResolveParams.h"
#include "Misc/NotifyHook.h"
#include "MovieSceneTrack.h"
#include "LevelSequenceEditorSubsystem.generated.h"

#define UE_API LEVELSEQUENCEEDITOR_API

class FUICommandList;
class ISequencer;
class UMovieSceneTrack;
struct FMovieSceneBindingProxy;
struct FMovieScenePasteBindingsParams;
struct FMovieScenePasteFoldersParams;
struct FMovieScenePasteSectionsParams;
struct FMovieScenePasteTracksParams;
struct FBakingAnimationKeySettings;
struct FSequencerChangeBindingInfo;
struct FMovieScenePossessable;

DECLARE_LOG_CATEGORY_EXTERN(LogLevelSequenceEditor, Log, All);

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequenceGlobalTimeChanged);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequencePlayEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequenceStopEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequenceRecordEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequenceBeginScrubbingEvent);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnLevelSequenceEndScrubbingEvent);

class ACineCameraActor;
class FExtender;
class FMenuBuilder;
class UMovieSceneCompiledDataManager;
class UMovieSceneFolder;
class UMovieSceneSection;
class UMovieSceneSequence;
class USequencerModuleScriptingLayer;
class IDetailsView;
class USequencerCurveEditorObject;
class UMovieSceneCustomBinding;
class IMenu;
class UMovieSceneTrackRowMetadataHelper;
class SWidget;

USTRUCT(BlueprintType)
struct FMovieSceneScriptingParams
{
	GENERATED_BODY()
		
	FMovieSceneScriptingParams() {}

	UPROPERTY(BlueprintReadWrite, Category = "Movie Scene")
	EMovieSceneTimeUnit TimeUnit = EMovieSceneTimeUnit::DisplayRate;
};

// Helper struct for Binding Properties UI for locators.
USTRUCT()
struct FMovieSceneBindingPropertyInfo
{
	GENERATED_BODY()

	// Locator for the entry
	UPROPERTY(EditAnywhere, Category = "Default", meta=(AllowedLocators="Actor, UsdPrim", DisplayName="Actor"))
	FUniversalObjectLocator Locator;

	// Flags for how to resolve the locator
	UPROPERTY()
	ELocatorResolveFlags ResolveFlags = ELocatorResolveFlags::None;

	UPROPERTY(Instanced, VisibleAnywhere, Category = "Default", meta=(EditInline, AllowEditInlineCustomization, DisplayName="Custom Binding Type"))
	TObjectPtr<UMovieSceneCustomBinding> CustomBinding = nullptr;
};

// Helper UObject for editing arrays of locators for object bindings. A UObject instead of a UStruct because we need to support instanced sub objects
UCLASS()
class UMovieSceneBindingPropertyInfoList : public UObject
{
	GENERATED_BODY()
public:
	// List of locator info for a particular binding
	UPROPERTY(EditAnywhere, Category = "Binding Properties")
	TArray<FMovieSceneBindingPropertyInfo> Bindings;

	// The Sequencer that the Bindings are sourced from
	TWeakPtr<ISequencer> Sequencer;
};

/**
* ULevelSequenceEditorSubsystem
* Subsystem for level sequence editor related utilities to scripts
*/
UCLASS(MinimalAPI)
class ULevelSequenceEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

	UE_API void OnSequencerCreated(TSharedRef<ISequencer> InSequencer);

	UE_API void OnSequencerClosed(TSharedRef<ISequencer> InSequencer);

	/** Retrieve the scripting layer */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	UE_API USequencerModuleScriptingLayer* GetScriptingLayer();

	/** Retrieve the curve editor */
	UFUNCTION(BlueprintPure, Category = "Level Sequence Editor")
	UE_API USequencerCurveEditorObject* GetCurveEditor();

	/** Called when the global time changes in the active sequencer */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequenceGlobalTimeChanged OnGlobalTimeChanged;

	/** Called when the active sequencer begins playing */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequencePlayEvent OnPlayEvent;

	/** Called when the active sequencer stops playing */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequenceStopEvent OnStopEvent;

	/** Called when the active sequencer toggles recording */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequenceRecordEvent OnRecordEvent;

	/** Called when the user begins scrubbing in the active sequencer */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequenceBeginScrubbingEvent OnBeginScrubbingEvent;

	/** Called when the user ends scrubbing in the active sequencer */
	UPROPERTY(BlueprintAssignable, Category = "Level Sequence Editor")
	FOnLevelSequenceEndScrubbingEvent OnEndScrubbingEvent;

	/** Add existing actors to Sequencer. Tracks will be automatically added based on default track settings. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API TArray<FMovieSceneBindingProxy> AddActors(const TArray<AActor*>& Actors);

	/** Add a new binding to this sequence that will spawn the specified object. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API FMovieSceneBindingProxy AddSpawnableFromInstance(UMovieSceneSequence* Sequence, UObject* ObjectToSpawn);

	/** Add a new binding to this sequence that will spawn the specified class. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API FMovieSceneBindingProxy AddSpawnableFromClass(UMovieSceneSequence* Sequence, UClass* ClassToSpawn);

	/** Create a cine camera actor and add it to Sequencer */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API FMovieSceneBindingProxy CreateCamera(bool bSpawnable, ACineCameraActor*& OutActor);

	/** 
	* Convert to spawnable. If there are multiple objects assigned to the possessable, multiple spawnables will be created. 
	* For level sequences, the bindings created will be custom bindings of type UMovieSceneSpawnableActorBinding.
	*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API TArray<FMovieSceneBindingProxy> ConvertToSpawnable(const FMovieSceneBindingProxy& ObjectBinding);

	/** Convert to possessable. If there are multiple objects assigned to the spawnable. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API FMovieSceneBindingProxy ConvertToPossessable(const FMovieSceneBindingProxy& ObjectBinding);

	/** Convert to a custom binding of the given binding type*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API FMovieSceneBindingProxy ConvertToCustomBinding(const FMovieSceneBindingProxy& ObjectBinding, UPARAM(meta = (AllowAbstract = "false")) TSubclassOf<UMovieSceneCustomBinding> BindingType);

	/** In the case that the given binding proxy holds custom bindings, returns an array of the binding objects so properties can be accessed. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API TArray<UMovieSceneCustomBinding*> GetCustomBindingObjects(const FMovieSceneBindingProxy& ObjectBinding);

	/** Returns all of the bindings in the sequence of the given custom type. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API TArray<FMovieSceneBindingProxy> GetCustomBindingsOfType(TSubclassOf<UMovieSceneCustomBinding> CustomBindingType);

	/* Returns the custom binding type for the given binding, or nullptr for possessables*/
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API TSubclassOf<UMovieSceneCustomBinding> GetCustomBindingType(const FMovieSceneBindingProxy& ObjectBinding);

	/* Sets the actor class for the spawnable or replaceable template, in the case those binding types support templates. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool ChangeActorTemplateClass(const FMovieSceneBindingProxy& ObjectBinding, TSubclassOf<AActor> ActorClass);

	/* Save the default state of the spawnable. */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void SaveDefaultSpawnableState(const FMovieSceneBindingProxy& ObjectBinding);

	/** 
	 * Copy folders 
	 * The copied folders will be saved to the clipboard as well as assigned to the ExportedText string. 
	 * The ExportedTest string can be used in conjunction with PasteFolders if, for example, pasting copy/pasting multiple 
	 * folders without relying on a single clipboard. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void CopyFolders(const TArray<UMovieSceneFolder*>& Folders, FString& FoldersExportedText, FString& ObjectsExportedText, FString& TracksExportedText);

	UE_DEPRECATED(5.5, "CopyFolders now gathers objects and tracks within the folders. Please use CopyFolders that outputs ObjectsExportedText and TracksExportedText")
	UE_API void CopyFolders(const TArray<UMovieSceneFolder*>& Folders, FString& FoldersExportedText);

	/** 
	 * Paste folders 
	 * Paste folders from the given TextToImport string (used in conjunction with CopyFolders). 
	 * If TextToImport is empty, the contents of the clipboard will be used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool PasteFolders(const FString& TextToImport, FMovieScenePasteFoldersParams PasteFoldersParams, TArray<UMovieSceneFolder*>& OutFolders);

	/**
	 * Copy sections
	 * The copied sections will be saved to the clipboard as well as assigned to the ExportedText string.
	 * The ExportedTest string can be used in conjunction with PasteSections if, for example, pasting copy/pasting multiple
	 * sections without relying on a single clipboard.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void CopySections(const TArray<UMovieSceneSection*>& Sections, FString& ExportedText);

	/**
	 * Paste sections
	 * Paste sections from the given TextToImport string (used in conjunction with CopySections).
	 * If TextToImport is empty, the contents of the clipboard will be used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool PasteSections(const FString& TextToImport, FMovieScenePasteSectionsParams PasteSectionsParams, TArray<UMovieSceneSection*>& OutSections);

	/**
	 * Copy tracks
	 * The copied tracks will be saved to the clipboard as well as assigned to the ExportedText string.
	 * The ExportedTest string can be used in conjunction with PasteTracks if, for example, pasting copy/pasting multiple
	 * tracks without relying on a single clipboard.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void CopyTracks(const TArray<UMovieSceneTrack*>& Tracks, FString& ExportedText);

	/**
	 * Paste tracks
	 * Paste tracks from the given TextToImport string (used in conjunction with CopyTracks).
	 * If TextToImport is empty, the contents of the clipboard will be used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool PasteTracks(const FString& TextToImport, FMovieScenePasteTracksParams PasteTracksParams, TArray<UMovieSceneTrack*>& OutTracks);

	/**
	 * Copy bindings
	 * The copied bindings will be saved to the clipboard as well as assigned to the ExportedText string.
	 * The ExportedTest string can be used in conjunction with PasteBindings if, for example, pasting copy/pasting multiple
	 * bindings without relying on a single clipboard.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void CopyBindings(const TArray<FMovieSceneBindingProxy>& Bindings, FString& ExportedText);

	/**
	 * Paste bindings
	 * Paste bindings from the given TextToImport string (used in conjunction with CopyBindings).
	 * If TextToImport is empty, the contents of the clipboard will be used.
	 */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool PasteBindings(const FString& TextToImport, FMovieScenePasteBindingsParams PasteBindingsParams, TArray<FMovieSceneBindingProxy>& OutObjectBindings);

	/** Snap sections to timeline using source timecode */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void SnapSectionsToTimelineUsingSourceTimecode(const TArray<UMovieSceneSection*>& Sections);

	/** Sync section using source timecode */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void SyncSectionsUsingSourceTimecode(const TArray<UMovieSceneSection*>& Sections);

	/** Bake transform */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API bool BakeTransformWithSettings(const TArray<FMovieSceneBindingProxy>& ObjectBindings, const FBakingAnimationKeySettings& InSettings, const FMovieSceneScriptingParams& Params = FMovieSceneScriptingParams());

	/** Attempts to automatically fix up broken actor references in the current scene */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void FixActorReferences();

	/** Assigns the given actors to the binding */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void AddActorsToBinding(const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);

	/** Replaces the binding with the given actors */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void ReplaceBindingWithActors(const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);

	/** Removes the given actors from the binding */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void RemoveActorsFromBinding(const TArray<AActor*>& Actors, const FMovieSceneBindingProxy& ObjectBinding);

	/** Remove all bound actors from this track */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void RemoveAllBindings(const FMovieSceneBindingProxy& ObjectBinding);

	/** Remove missing objects bound to this track */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void RemoveInvalidBindings(const FMovieSceneBindingProxy& ObjectBinding);

	/** Rebind the component binding to the requested component */
	UFUNCTION(BlueprintCallable, Category = "Level Sequence Editor")
	UE_API void RebindComponent(const TArray<FMovieSceneBindingProxy>& ComponentBindings, const FName& ComponentName);

	// Refreshes the binding details when the bindings change in the menu
	UE_API void RefreshBindingDetails(IDetailsView* DetailsView, FGuid ObjectBindingID);

	// Refreshes the track row metadata details when the track row metadata changes in the menu
	UE_API void RefreshTrackRowMetadataDetails(IDetailsView* DetailsView);

private:

	// Used by binding properties menu
	struct FBindingPropertiesNotifyHook : FNotifyHook
	{
		UMovieSceneSequence* ObjectToModify = nullptr;
		FBindingPropertiesNotifyHook() {}

		FBindingPropertiesNotifyHook(UMovieSceneSequence* InObjectToModify) : ObjectToModify(InObjectToModify) {}

		virtual void NotifyPreChange(FProperty* PropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	};

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneBindingPropertyInfoList>> BindingPropertyInfoLists;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneTrackRowMetadataHelper>> TrackRowMetadataHelperList;
	
	FBindingPropertiesNotifyHook NotifyHook;

private:

	UE_API void AddBindingDetailCustomizations(TSharedRef<IDetailsView> DetailsView, TSharedPtr<ISequencer> ActiveSequencer, FGuid BindingGuid);
	UE_API void AddTrackRowMetadataCustomizations(TSharedRef<IDetailsView> DetailsView, TSharedPtr<ISequencer> ActiveSequencer, UMovieSceneSequence* Sequence);
	UE_API void OnTrackRowMetadataMenuBeingDestroyed(const TSharedRef<IMenu>& Menu, TSharedRef<IDetailsView> DetailsView);

	UE_API TSharedPtr<ISequencer> GetActiveSequencer();
	
	UE_API void RebindComponentInternal(const FName& ComponentName);

	UE_API void AddBindingPropertiesMenu(FMenuBuilder& MenuBuilder);
	
	UE_API void FillDirectorBlueprintBindingSubMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToChange, bool bConvert, TFunction<void()> OnBindingChanged, const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType);
	UE_API void PopulateQuickBindSubMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToChange, bool bConvert, TFunction<void()> OnBindingChanged, const TSubclassOf<UMovieSceneCustomBinding>& CustomBindingType);
	UE_API void FillBindingClassSubMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToChange, bool bConvert, TFunction<void()> OnBindingChanged, const TArray<const TSubclassOf<UMovieSceneCustomBinding>>& UserCustomBindingTypes);
	UE_API void ChangeBindingTypes(const TSharedRef<ISequencer>& InSequencer
		, const TArray<FSequencerChangeBindingInfo>& InBindingsToChange
		, TFunction<FMovieScenePossessable* (FGuid, int32)> InDoChangeType
		, TFunction<void()> InOnBindingChanged);

	UE_API void AddTrackRowMetadataMenu(FMenuBuilder& MenuBuilder);
public:
	UE_API void AddBindingPropertiesSidebar(FMenuBuilder& MenuBuilder);

	UE_API TSharedRef<SWidget> BuildBindingPropertiesDetailsWidget(TSharedRef<ISequencer> Sequencer);
	UE_API void RefreshBindingDetails(TSharedRef<ISequencer> Sequencer, IDetailsView* DetailsView, FGuid ObjectBindingID);

	/* Creates a menu for changing or converting a binding type. If bConvert is true, it will only show types that state they are able to be converted to from the passed in bindings
	and will attempt to convert them. If bConvert is false, it will change the binding type and reset to a default binding of that type.*/
	UE_API void AddChangeBindingTypeMenu(FMenuBuilder& MenuBuilder, TSharedRef<ISequencer> Sequencer, const TArray<FSequencerChangeBindingInfo>& BindingsToChange, bool bConvert, TFunction<void()> OnBindingChanged);

	/** Add the default movie scene tracks for the given actor to its binding in the sequencer's focused sequence. */
	UE_API void AddDefaultTracksForActor(TSharedRef<ISequencer> InSequencer, AActor& Actor, const FGuid Binding);

private:
	void OnFinishedChangingLocators(const FPropertyChangedEvent& PropertyChangedEvent, TWeakPtr<ISequencer> WeakSequencer, TSharedRef<IDetailsView> DetailsView, FGuid ObjectBindingID);

	UE_API void OnFinishedChangingTrackRowMetadata(const FPropertyChangedEvent& PropertyChangedEvent, TSharedRef<IDetailsView> DetailsView);

	UE_API void GetRebindComponentNames(TArray<FName>& OutComponentNames);
	UE_API void RebindComponentMenu(FMenuBuilder& MenuBuilder);

	UMovieSceneBindingPropertyInfoList* GetBindingPropertyInfoList(TSharedRef<ISequencer> Sequencer);

	UE_API void CreatePlaybackDelegates(TSharedRef<ISequencer> InSequencer);

	void HandleGlobalTimeChanged();
	void HandlePlayEvent();
	void HandleStopEvent();
	void HandleRecordEvent();
	void HandleBeginScrubbingEvent();
	void HandleEndScrubbingEvent();

	struct FSequencerPlaybackDelegateHandles
	{
		FDelegateHandle GlobalTimeChanged;
		FDelegateHandle PlayEvent;
		FDelegateHandle StopEvent;
		FDelegateHandle RecordEvent;
		FDelegateHandle BeginScrubbing;
		FDelegateHandle EndScrubbing;
	};
	TMap<TWeakPtr<ISequencer>, FSequencerPlaybackDelegateHandles> SequencerPlaybackDelegateHandles;

	FDelegateHandle OnSequencerCreatedHandle;

	/* List of sequencers that have been created */
	TArray<TWeakPtr<ISequencer>> Sequencers;

	/* Map of curve editors with their sequencers*/
	TMap<TWeakPtr<ISequencer>, TObjectPtr<USequencerCurveEditorObject>> CurveEditorObjects;
	/* property array of the curve editors*/
	UPROPERTY()
	TArray<TObjectPtr<USequencerCurveEditorObject>> CurveEditorArray;

	TSharedPtr<FExtender> RebindComponentMenuExtender;
	TSharedPtr<FExtender> SidebarMenuExtender;

	friend class FMovieSceneBindingPropertyInfoListCustomization;
};

#undef UE_API
