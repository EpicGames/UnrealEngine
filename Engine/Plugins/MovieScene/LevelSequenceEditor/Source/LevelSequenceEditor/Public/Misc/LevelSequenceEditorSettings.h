// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceEditorSettings.generated.h"

#define UE_API LEVELSEQUENCEEDITOR_API

USTRUCT()
struct FLevelSequencePropertyTrackSettings
{
	GENERATED_BODY()

	/** Optional ActorComponent tag (when keying a component property). */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString ComponentPath;

	/** Path to the keyed property within the Actor or ActorComponent. */
	UPROPERTY(config, EditAnywhere, Category=PropertyTrack)
	FString PropertyPath;
};


USTRUCT()
struct FLevelSequenceTrackSettings
{
	GENERATED_BODY()

	/** The Actor class to create movie scene tracks for. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/Engine.Actor"))
	FSoftClassPath MatchingActorClass;

	/** List of movie scene track classes to be added automatically. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/MovieScene.MovieSceneTrack"))
	TArray<FSoftClassPath> DefaultTracks;

	/** List of movie scene track classes not to be added automatically. */
	UPROPERTY(config, noclear, EditAnywhere, Category=TrackSettings, meta=(MetaClass="/Script/MovieScene.MovieSceneTrack"))
	TArray<FSoftClassPath> ExcludeDefaultTracks;

	/** List of property names for which movie scene tracks will be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	TArray<FLevelSequencePropertyTrackSettings> DefaultPropertyTracks;

	/** List of property names for which movie scene tracks will not be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=TrackSettings)
	TArray<FLevelSequencePropertyTrackSettings> ExcludeDefaultPropertyTracks;
};


/**
 * Level Sequence Editor settings.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class ULevelSequenceEditorSettings
	: public UObject
{
	GENERATED_BODY()

public:

	UE_API ULevelSequenceEditorSettings(const FObjectInitializer& ObjectInitializer);

	/** Specifies class properties for which movie scene tracks will be created automatically. */
	UPROPERTY(config, EditAnywhere, Category=Tracks)
	TArray<FLevelSequenceTrackSettings> TrackSettings;

	/** Specifies whether to automatically bind an active sequencer UI to PIE worlds. */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bAutoBindToPIE;

	/** Specifies whether to automatically bind an active sequencer UI to simulate worlds. */
	UPROPERTY(config, EditAnywhere, Category=Playback)
	bool bAutoBindToSimulate;
};

/**
 * Level Sequence With Shots Settings.
 */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings)
class ULevelSequenceWithShotsSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/** Sequence With Shots name. */
	UPROPERTY(config, DisplayName="Name", EditAnywhere, Category=SequenceWithShots)
	FString Name;

	/** Sequence With Shots suffix. */
	UPROPERTY(config, DisplayName="Suffix", EditAnywhere, Category=SequenceWithShots)
	FString Suffix;

	/** Sequence With Shots path. */
	UPROPERTY(config, DisplayName="Base Path", EditAnywhere, Category=SequenceWithShots, meta=(ContentDir))
	FDirectoryPath BasePath;

	/** Sequence With Shots number of shots. */
	UPROPERTY(config, DisplayName="Number of Shots", EditAnywhere, Category=SequenceWithShots, meta = (UIMin = "1", UIMax = "100"))
	uint32 NumShots;

	/** Sequence With Shots level sequence to duplicate when creating shots. */
	UPROPERTY(Transient, DisplayName="Sequence to Duplicate", EditAnywhere, Category=SequenceWithShots)
	TSoftObjectPtr<class ULevelSequence> SequenceToDuplicate;

	/** Array of sub sequence names, each will result in a level sequence asset in the shot. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category= SequenceWithShots)
	TArray<FName> SubSequenceNames;

	/** Whether to instance sub sequences based on the first created sub sequences. */
	UPROPERTY(config, DisplayName="Instance Sub Sequences", EditAnywhere, Category=SequenceWithShots)
	bool bInstanceSubSequences;
};

#undef UE_API
