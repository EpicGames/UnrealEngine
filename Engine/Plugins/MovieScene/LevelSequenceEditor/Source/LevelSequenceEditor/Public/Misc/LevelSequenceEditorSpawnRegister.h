// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequenceSpawnRegister.h"
#include "UObject/ObjectKey.h"

#define UE_API LEVELSEQUENCEEDITOR_API

struct FNewSpawnable;

class IMovieScenePlayer;
class ISequencer;
class UMovieScene;
class UMovieSceneSequence;
class FObjectPreSaveContext;

/**
 * Spawn register used in the editor to add some usability features like maintaining selection states, and projecting spawned state onto spawnable defaults
 */
class FLevelSequenceEditorSpawnRegister
	: public FLevelSequenceSpawnRegister
{
public:

	/** Constructor */
	UE_API FLevelSequenceEditorSpawnRegister();

	/** Destructor. */
	UE_API ~FLevelSequenceEditorSpawnRegister();

public:

	UE_API void SetSequencer(const TSharedPtr<ISequencer>& Sequencer);

public:

	// FLevelSequenceSpawnRegister interface
	// Prevents clang warning on the old deprecated SpawnObject overload that takes a FMovieSceneSpawnable
	using FLevelSequenceSpawnRegister::SpawnObject;
	UE_API virtual UObject* SpawnObject(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef Template, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, int32 BindingIndex);
	UE_API virtual void PreDestroyObject(UObject& Object, const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID) override;
#if WITH_EDITOR
	UE_API virtual void SaveDefaultSpawnableState(const FGuid& BindingId, int32 BindingIndex, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState) override;
	UE_API virtual TValueOrError<FNewSpawnable, FText> CreateNewSpawnableType(UObject& SourceObject, UMovieScene& OwnerMovieScene, UActorFactory* ActorFactory = nullptr) override;
	UE_API virtual void SetupDefaultsForSpawnable(UObject* SpawnedObject, const FGuid& Guid, const TOptional<FTransformData>& TransformData, TSharedRef<ISequencer> Sequencer, USequencerSettings* Settings) override;
	UE_API virtual void HandleConvertPossessableToSpawnable(UObject* OldObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, TOptional<FTransformData>& OutTransformData) override;
	UE_API virtual bool CanConvertSpawnableToPossessable(FMovieSceneSpawnable& Spawnable) const override;
#endif

private:

	/** Called when the editor selection has changed. */
	UE_API void HandleActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

	/** Saves the default state for the specified spawnable, if an instance for it currently exists */
	UE_API void SaveDefaultSpawnableStateImpl(const FGuid& BindingId, int32 BindingIndex, UMovieSceneSequence* Sequence, UObject* SpawnedObject, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);

	/** Called from the editor when a blueprint object replacement has occurred */
	UE_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap);

	/** Called whenever an object is modified in the editor */
	UE_API void OnObjectModified(UObject* ModifiedObject);

	/** Called before an object is saved in the editor */
	UE_API void OnPreObjectSaved(UObject* Object, FObjectPreSaveContext SaveContext);

	/** Called on pre/post GC */
	UE_API void UpdateIsEngineCollectingGarbage(bool bIsCollectingGarbage);


private:

	struct FTrackedObjectState
	{
		FTrackedObjectState(FMovieSceneSequenceIDRef InTemplateID, const FGuid& InObjectBindingID, int32 InBindingIndex=0) : TemplateID(InTemplateID), ObjectBindingID(InObjectBindingID), BindingIndex(InBindingIndex), bHasBeenModified(false) {}

		/** The sequence ID that spawned this object */
		FMovieSceneSequenceID TemplateID;

		/** The object binding ID of the object in the template */
		FGuid ObjectBindingID;

		/** The index of the binding being tracked*/
		int32 BindingIndex = 0;

		/** true if this object has been modified since it was spawned and is different from the current object template */
		bool bHasBeenModified;
	};

private:

	/** Handles for delegates that we've bound to. */
	FDelegateHandle OnActorSelectionChangedHandle;

	/** Set of spawn register keys for objects that should be selected if they are spawned. */
	TSet<FMovieSceneSpawnRegisterKey> SelectedSpawnedObjects;

	/** Map from a sequenceID to an array of objects that have been tracked */
	TMap<FObjectKey, FTrackedObjectState> TrackedObjects;

	/** Set of UMovieSceneSequences that this register has spawned objects for that are modified */
	TSet<FObjectKey> SequencesWithModifiedObjects;

	/** True if we should clear the above selection cache when the editor selection has been changed. */
	bool bShouldClearSelectionCache;

	/** Weak pointer to the active sequencer. */
	TWeakPtr<ISequencer> WeakSequencer;

	/** Handle to a delegate that is bound to FCoreUObjectDelegates::OnObjectModified to harvest changes to spawned objects. */
	FDelegateHandle OnObjectModifiedHandle;

	/** Handle to a delegate that is bound to FCoreUObjectDelegates::OnObjectPreSave to harvest changes to spawned objects. */
	FDelegateHandle OnObjectSavedHandle;

	/** True when GCing */
	bool bIsEngineCollectingGarbage;

	/** Handle to a delegate that is bound to FCoreUObjectDelegates::OnPre/PostGarbageCollectHandle to disable saving changes while GCing. */
	FDelegateHandle OnPreGarbageCollectHandle;
	FDelegateHandle OnPostGarbageCollectHandle;
};

#undef UE_API
