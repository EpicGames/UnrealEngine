// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakInterfacePtr.h"

class SWidget;

class ALevelSequenceActor;
class IMovieScenePlaybackClient;
class ULevelSequence;

#define UE_API LEVELSEQUENCEEDITOR_API

/**
 * Class that manages the current playback context that a level-sequence editor should use for playback
 */
class FLevelSequencePlaybackContext : public TSharedFromThis<FLevelSequencePlaybackContext>
{
public:

	UE_API FLevelSequencePlaybackContext(ULevelSequence* InLevelSequence);
	UE_API ~FLevelSequencePlaybackContext();

	/**
	 * Gets the level sequence for which we are trying to find the context.
	 */
	UE_API ULevelSequence* GetLevelSequence() const;

	/**
	 * Build a world picker widget that allows the user to choose a world, and exit the auto-bind settings
	 */
	UE_API TSharedRef<SWidget> BuildWorldPickerCombo();

	/**
	 * Resolve the current world context pointer. Can never be nullptr.
	 */
	UE_API UObject* GetPlaybackContext() const;

	/**
	 * Returns GetPlaybackContext as a plain object.
	 */
	UE_API UObject* GetPlaybackContextAsObject() const;

	/**
	 * Resolve the current playback client. May be nullptr.
	 */
	UE_API UObject* GetPlaybackClientAsUObject() const;

	UE_DEPRECATED(5.8, "GetPlaybackClient has been deprecated, please use GetPlaybackClientAsUObject")
	UE_API ALevelSequenceActor* GetPlaybackClient() const;

	/**
	 * Returns GetPlaybackClient as an interface pointer.
	 */
	UE_API IMovieScenePlaybackClient* GetPlaybackClientAsInterface() const;

	/**
	 * Retrieve all the event contexts for the current world
	 */
	UE_API TArray<UObject*> GetEventContexts() const;

	/**
	 * Specify a new world to use as the context. Persists until the next PIE or map change event.
	 * May be null, in which case the context will be recomputed automatically
	 */
	UE_API void OverrideWith(UWorld* InNewContext, IMovieScenePlaybackClient* InNewClient);

private:

	using FContextAndClient = TTuple<UWorld*, IMovieScenePlaybackClient*>;

	/**
	 * Compute the new playback context based on the user's current auto-bind settings.
	 * Will use the first encountered PIE or Simulate world if possible, else the Editor world as a fallback
	 */
	static UE_API FContextAndClient ComputePlaybackContextAndClient(const ULevelSequence* InLevelSequence);

	/**
	 * Update the cached context and client pointers if needed.
	 */
	UE_API void UpdateCachedContextAndClient() const;

	/**
	 * Gets both the context and client.
	 */
	//TTuple<UWorld*, IMovieScenePlaybackClient*>
	UE_API FContextAndClient GetPlaybackContextAndClient() const;

	UE_API void OnPieEvent(bool);
	UE_API void OnMapChange(uint32);
	UE_API void OnWorldListChanged(UWorld*);

private:

	/** Level sequence that we should find a context for */
	TWeakObjectPtr<ULevelSequence> LevelSequence;

	/** Mutable cached context pointer */
	mutable TWeakObjectPtr<UWorld> WeakCurrentContext;

	/** Mutable cached client pointer */
	mutable TWeakInterfacePtr<IMovieScenePlaybackClient> WeakCurrentClient;
};

#undef UE_API
