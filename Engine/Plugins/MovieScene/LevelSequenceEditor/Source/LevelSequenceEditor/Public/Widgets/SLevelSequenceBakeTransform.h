// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BakingAnimationKeySettings.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/StructOnScope.h"

class ISequencer;
class IStructureDetailsView;
class SWindow;

#define UE_API LEVELSEQUENCEEDITOR_API

DECLARE_DELEGATE_RetVal_OneParam(FReply, SLevelSequenceBakeTransformOnBake, FBakingAnimationKeySettings);

/** Widget that displays bake options for baking transforms */
class SLevelSequenceBakeTransform : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SLevelSequenceBakeTransform)
		: _Sequencer(nullptr)
		{
		}
		SLATE_ARGUMENT(ISequencer*, Sequencer)
		SLATE_ARGUMENT(FBakingAnimationKeySettings, Settings)
		SLATE_EVENT(SLevelSequenceBakeTransformOnBake, OnBake)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);
	virtual ~SLevelSequenceBakeTransform() override = default;

	UE_API FReply OpenDialog(bool bModal = true);
	void CloseDialog();

private:

	//used for setting up the details
	TSharedPtr<TStructOnScope<FBakingAnimationKeySettings>> Settings;

	ISequencer* Sequencer;

	TWeakPtr<SWindow> DialogWindow;
	TSharedPtr<IStructureDetailsView> DetailsView;
};

#undef UE_API
