// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LevelSequence.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API LEVELSEQUENCEEDITOR_API

/** Widget for viewing and setting the Favorite Rating on LevelSequences. */
class SLevelSequenceFavoriteRating : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLevelSequenceFavoriteRating) 
		: _LevelSequence(nullptr)
		{}

		/** The LevelSequence object to focus the widget on */
		SLATE_ARGUMENT( TWeakObjectPtr<ULevelSequence>, LevelSequence )
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:
	/** The LevelSequence object the widget is focused on. Only a single sequence can be focused. 
	 *  This could be changed using the Undetermined state and special multi icons. 
	 */
	TWeakObjectPtr<ULevelSequence> WeakLevelSequence;
};

#undef UE_API
